// Copyright Epic Games, Inc. All Rights Reserved.

#include "TechAudioToolsFloatMapping.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TechAudioToolsFloatMapping)

FFloatInterval UTechAudioToolsFloatMapping::GetRange(const ETechAudioToolsMappingEndpoint Endpoint) const
{
	switch (MappingType)
	{
	case ETechAudioToolsFloatMappingType::Default:
		return Range;

	case ETechAudioToolsFloatMappingType::MapRange:
		return Endpoint == ETechAudioToolsMappingEndpoint::Display ? DisplayRange : SourceRange;

	case ETechAudioToolsFloatMappingType::Volume:
		{
			const FFloatInterval VolumeDisplayRange = DisplayVolumeUnits == ETechAudioToolsVolumeUnit::Decibels ? DisplayRange_Decibels : DisplayRange_LinearGain;
			if (Endpoint == ETechAudioToolsMappingEndpoint::Display || SourceVolumeUnits == DisplayVolumeUnits)
			{
				return VolumeDisplayRange;
			}

			return TechAudioTools::ConvertRange<ETechAudioToolsVolumeUnit>(DisplayVolumeUnits, SourceVolumeUnits, VolumeDisplayRange);
		}

	case ETechAudioToolsFloatMappingType::Pitch:
		{
			const FFloatInterval PitchDisplayRange = DisplayPitchUnits == ETechAudioToolsPitchUnit::Semitones ? DisplayRange_Semitones : DisplayRange_FrequencyMultiplier;
			if (Endpoint == ETechAudioToolsMappingEndpoint::Display || SourcePitchUnits == DisplayPitchUnits)
			{
				return PitchDisplayRange;
			}

			return TechAudioTools::ConvertRange<ETechAudioToolsPitchUnit>(DisplayPitchUnits, SourcePitchUnits, PitchDisplayRange);
		}
	}

	ensureMsgf(false, TEXT("Unhandled MappingType"));
	return FFloatInterval(0.f, 1.f);
}

ETechAudioToolsFloatUnit UTechAudioToolsFloatMapping::GetUnits(const ETechAudioToolsMappingEndpoint Endpoint) const
{
	using namespace TechAudioTools;

	switch (MappingType)
	{
	case ETechAudioToolsFloatMappingType::Default:
	case ETechAudioToolsFloatMappingType::MapRange:
		return DisplayUnits;

	case ETechAudioToolsFloatMappingType::Volume:
		{
			const bool bShowDecibels = Endpoint == ETechAudioToolsMappingEndpoint::Display ? DisplayVolumeUnits == ETechAudioToolsVolumeUnit::Decibels : SourceVolumeUnits == ETechAudioToolsVolumeUnit::Decibels;
			return bShowDecibels ? ETechAudioToolsFloatUnit::Decibels : ETechAudioToolsFloatUnit::Multiplier;
		}

	case ETechAudioToolsFloatMappingType::Pitch:
		{
			const bool bShowSemitones = Endpoint == ETechAudioToolsMappingEndpoint::Display ? DisplayPitchUnits == ETechAudioToolsPitchUnit::Semitones : SourcePitchUnits == ETechAudioToolsPitchUnit::Semitones;
			return bShowSemitones ? ETechAudioToolsFloatUnit::Semitones : ETechAudioToolsFloatUnit::Multiplier;
		}
	}

	ensureMsgf(false, TEXT("Unhandled MappingType"));
	return ETechAudioToolsFloatUnit::None;
}

float UTechAudioToolsFloatMapping::SourceToDisplay(const float InSource) const
{
	switch (MappingType)
	{
	case ETechAudioToolsFloatMappingType::Default:
		return InSource;

	case ETechAudioToolsFloatMappingType::MapRange:
		{
			const float Normalized = FMath::GetRangePct(GetSourceMin(), GetSourceMax(), InSource);
			return FMath::Lerp(GetDisplayMin(), GetDisplayMax(), Normalized);
		}

	case ETechAudioToolsFloatMappingType::Volume:
		return TechAudioTools::ConvertUnit(SourceVolumeUnits, DisplayVolumeUnits, InSource);

	case ETechAudioToolsFloatMappingType::Pitch:
		return TechAudioTools::ConvertUnit(SourcePitchUnits, DisplayPitchUnits, InSource);
	}

	ensureMsgf(false, TEXT("Unhandled MappingType"));
	return InSource;
}

float UTechAudioToolsFloatMapping::DisplayToSource(const float InDisplay) const
{
	switch (MappingType)
	{
	case ETechAudioToolsFloatMappingType::Default:
		return InDisplay;

	case ETechAudioToolsFloatMappingType::MapRange:
		{
			const float Normalized = FMath::GetRangePct(GetDisplayMin(), GetDisplayMax(), InDisplay);
			return FMath::Lerp(GetSourceMin(), GetSourceMax(), Normalized);
		}

	case ETechAudioToolsFloatMappingType::Volume:
		return TechAudioTools::ConvertUnit(DisplayVolumeUnits, SourceVolumeUnits, InDisplay);

	case ETechAudioToolsFloatMappingType::Pitch:
		return TechAudioTools::ConvertUnit(DisplayPitchUnits, SourcePitchUnits, InDisplay);
	}

	ensureMsgf(false, TEXT("Unhandled MappingType"));
	return InDisplay;
}

float UTechAudioToolsFloatMapping::SourceToNormalized(const float InSource) const
{
	return FMath::GetRangePct(GetSourceMin(), GetSourceMax(), InSource);
}

float UTechAudioToolsFloatMapping::NormalizedToSource(const float InNormalized) const
{
	return FMath::Lerp(GetSourceMin(), GetSourceMax(), InNormalized);
}

float UTechAudioToolsFloatMapping::DisplayToNormalized(const float InDisplay) const
{
	return FMath::GetRangePct(GetDisplayMin(), GetDisplayMax(), InDisplay);
}

float UTechAudioToolsFloatMapping::NormalizedToDisplay(const float InNormalized) const
{
	return FMath::Lerp(GetDisplayMin(), GetDisplayMax(), InNormalized);
}
