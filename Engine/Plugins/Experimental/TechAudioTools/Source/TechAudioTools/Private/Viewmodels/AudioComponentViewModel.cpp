// Copyright Epic Games, Inc. All Rights Reserved.

#include "Viewmodels/AudioComponentViewModel.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AudioComponentViewModel)

void UAudioComponentViewModel::SetAudioComponent(UAudioComponent* InAudioComponent)
{
	if (AudioComponent == InAudioComponent)
	{
		return;
	}

	UnbindDelegates();
	AudioComponent = InAudioComponent;

	if (InAudioComponent)
	{
		SetPlayState(InAudioComponent->GetPlayState());
		UE_MVVM_SET_PROPERTY_VALUE(bIsVirtualized, InAudioComponent->IsVirtualized());
		BindDelegates();
	}
	else
	{
		UE_MVVM_SET_PROPERTY_VALUE(PlayState, EAudioComponentPlayState::Stopped);
		UE_MVVM_SET_PROPERTY_VALUE(bIsPlaying, false);
		UE_MVVM_SET_PROPERTY_VALUE(bIsStopped, true);
		UE_MVVM_SET_PROPERTY_VALUE(bIsFadingIn, false);
		UE_MVVM_SET_PROPERTY_VALUE(bIsFadingOut, false);
		UE_MVVM_SET_PROPERTY_VALUE(bIsVirtualized, false);
	}
}

void UAudioComponentViewModel::BeginDestroy()
{
	Super::BeginDestroy();
	UnbindDelegates();
}

void UAudioComponentViewModel::BindDelegates()
{
	if (AudioComponent.IsValid())
	{
		AudioComponent->OnAudioPlayStateChanged.AddUniqueDynamic(this, &ThisClass::SetPlayState);
		AudioComponent->OnAudioFinished.AddUniqueDynamic(this, &ThisClass::OnAudioFinished);
		AudioComponent->OnAudioVirtualizationChanged.AddUniqueDynamic(this, &ThisClass::OnVirtualizationChanged);
	}
}

void UAudioComponentViewModel::UnbindDelegates()
{
	if (AudioComponent.IsValid())
	{
		AudioComponent->OnAudioPlayStateChanged.RemoveDynamic(this, &ThisClass::SetPlayState);
		AudioComponent->OnAudioFinished.RemoveDynamic(this, &ThisClass::OnAudioFinished);
		AudioComponent->OnAudioVirtualizationChanged.RemoveDynamic(this, &ThisClass::OnVirtualizationChanged);
	}
}

void UAudioComponentViewModel::OnAudioFinished()
{
	if (AudioComponent.IsValid() && AudioComponent->bAutoDestroy != 0)
	{
		SetAudioComponent(nullptr);
	}
	else
	{
		SetPlayState(EAudioComponentPlayState::Stopped);
	}
}

void UAudioComponentViewModel::OnVirtualizationChanged(const bool bInIsVirtualized)
{
	UE_MVVM_SET_PROPERTY_VALUE(bIsVirtualized, bInIsVirtualized);
}

void UAudioComponentViewModel::SetPlayState(const EAudioComponentPlayState NewPlayState)
{
	if (UE_MVVM_SET_PROPERTY_VALUE(PlayState, NewPlayState))
	{
		UE_MVVM_SET_PROPERTY_VALUE(bIsPlaying, NewPlayState == EAudioComponentPlayState::Playing);
		UE_MVVM_SET_PROPERTY_VALUE(bIsStopped, NewPlayState == EAudioComponentPlayState::Stopped);
		UE_MVVM_SET_PROPERTY_VALUE(bIsFadingIn, NewPlayState == EAudioComponentPlayState::FadingIn);
		UE_MVVM_SET_PROPERTY_VALUE(bIsFadingOut, NewPlayState == EAudioComponentPlayState::FadingOut);
	}
}
