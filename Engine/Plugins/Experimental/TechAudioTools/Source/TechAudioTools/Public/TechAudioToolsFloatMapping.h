// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DSP/Dsp.h"
#include "TechAudioToolsTypes.h"

#include "TechAudioToolsFloatMapping.generated.h"

namespace TechAudioTools
{
	// Converts volume values between linear gain and decibels
	FORCEINLINE float ConvertUnit(const ETechAudioToolsVolumeUnit FromUnits, const ETechAudioToolsVolumeUnit ToUnits, const float Value)
	{
		if (FromUnits == ToUnits)
		{
			return Value;
		}

		return ToUnits == ETechAudioToolsVolumeUnit::Decibels ? Audio::ConvertToDecibels(Value) : Audio::ConvertToLinear(Value);
	}

	// Converts pitch values between semitones and frequency multiplier
	FORCEINLINE float ConvertUnit(const ETechAudioToolsPitchUnit FromUnits, const ETechAudioToolsPitchUnit ToUnits, const float Value)
	{
		if (FromUnits == ToUnits)
		{
			return Value;
		}

		return ToUnits == ETechAudioToolsPitchUnit::Semitones ? Audio::GetSemitones(Value) : Audio::GetFrequencyMultiplier(Value);
	}

	template <typename UnitT>
	FORCEINLINE FFloatInterval ConvertRange(UnitT FromUnits, UnitT ToUnits, FFloatInterval InputRange)
	{
		if (FromUnits == ToUnits)
		{
			return InputRange;
		}

		float RangeMin = ConvertUnit(FromUnits, ToUnits, InputRange.Min);
		float RangeMax = ConvertUnit(FromUnits, ToUnits, InputRange.Max);

		if (RangeMin > RangeMax)
		{
			Swap(RangeMin, RangeMax);
		}

		return FFloatInterval(RangeMin, RangeMax);
	}
} // namespace TechAudioTools

/**
 * Base class for any float value mapping that translates between two domains:
 * - Source domain: range used by an internal system (e.g. a MetaSound that uses linear gain or frequency multiplier).
 * - Display domain: how that value is shown or edited in the UI (e.g. sliders or knobs that use decibels or semitones).
 */
UCLASS(MinimalAPI, BlueprintType, EditInlineNew, DefaultToInstanced, CollapseCategories, DisplayName = "TechAudioTools Float Mapping")
class UTechAudioToolsFloatMapping : public UObject
{
	GENERATED_BODY()

public:
	// Selects the mapping mode.
	UPROPERTY(EditAnywhere, Category = "Range")
	ETechAudioToolsFloatMappingType MappingType = ETechAudioToolsFloatMappingType::Default;

	//~ Default

	// Default min/max values.
	UPROPERTY(EditAnywhere, Category = "Range", meta = (EditCondition = "MappingType == ETechAudioToolsFloatMappingType::Default", EditConditionHides))
	FFloatInterval Range { 0.f, 1.f };

	//~ Linear Remap

	// Range used by an internal system.
	UPROPERTY(EditAnywhere, Category = "Range", meta = (EditCondition = "MappingType == ETechAudioToolsFloatMappingType::MapRange", EditConditionHides))
	FFloatInterval SourceRange { 0.f, 1.f };

	// Range displayed in a user interface.
	UPROPERTY(EditAnywhere, Category = "Range", meta = (EditCondition = "MappingType == ETechAudioToolsFloatMappingType::MapRange", EditConditionHides))
	FFloatInterval DisplayRange { 0.f, 1.f };

	// Units used to label the display range.
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Map Range", meta = (EditCondition = "MappingType == ETechAudioToolsFloatMappingType::Default || MappingType == ETechAudioToolsFloatMappingType::MapRange", EditConditionHides))
	ETechAudioToolsFloatUnit DisplayUnits = ETechAudioToolsFloatUnit::None;

	//~ Volume

	// Units of the internal volume value.
	UPROPERTY(EditAnywhere, Category = "Volume", meta = (EditCondition = "MappingType == ETechAudioToolsFloatMappingType::Volume", EditConditionHides))
	ETechAudioToolsVolumeUnit SourceVolumeUnits = ETechAudioToolsVolumeUnit::LinearGain;

	// Units to show volume in a user interface.
	UPROPERTY(EditAnywhere, Category = "Volume", meta = (EditCondition = "MappingType == ETechAudioToolsFloatMappingType::Volume", EditConditionHides))
	ETechAudioToolsVolumeUnit DisplayVolumeUnits = ETechAudioToolsVolumeUnit::Decibels;

	// Range in decibels to display in a user interface.
	UPROPERTY(EditAnywhere, Category = "Volume", DisplayName = "Display Decibel Range", meta = (EditCondition = "MappingType == ETechAudioToolsFloatMappingType::Volume && DisplayVolumeUnits == ETechAudioToolsVolumeUnit::Decibels", EditConditionHides, ClampMin = "-60", ClampMax = "24"))
	FFloatInterval DisplayRange_Decibels { -60.f, 6.f };

	// Range in linear gain to display in a user interface.
	UPROPERTY(EditAnywhere, Category = "Volume", DisplayName = "Display Linear Gain Range", meta = (EditCondition = "MappingType == ETechAudioToolsFloatMappingType::Volume && DisplayVolumeUnits == ETechAudioToolsVolumeUnit::LinearGain", EditConditionHides, ClampMin = "0.0", ClampMax = "8.0"))
	FFloatInterval DisplayRange_LinearGain { 0.001f, 4.f };

	//~ Pitch

	// Units of the internal pitch value.
	UPROPERTY(EditAnywhere, Category = "Pitch", meta = (EditCondition = "MappingType == ETechAudioToolsFloatMappingType::Pitch", EditConditionHides))
	ETechAudioToolsPitchUnit SourcePitchUnits = ETechAudioToolsPitchUnit::Semitones;

	// Units to show pitch in a user interface.
	UPROPERTY(EditAnywhere, Category = "Pitch", meta = (EditCondition = "MappingType == ETechAudioToolsFloatMappingType::Pitch", EditConditionHides))
	ETechAudioToolsPitchUnit DisplayPitchUnits = ETechAudioToolsPitchUnit::Semitones;

	// Range in semitones to display in a user interface.
	UPROPERTY(EditAnywhere, Category = "Pitch", DisplayName = "Display Semitone Range", meta = (EditCondition = "MappingType == ETechAudioToolsFloatMappingType::Pitch && DisplayPitchUnits == ETechAudioToolsPitchUnit::Semitones", EditConditionHides, ClampMin = "-120", ClampMax = "120"))
	FFloatInterval DisplayRange_Semitones { -12.f, 12.f };

	// Range in frequency multiplier to display in a user interface.
	UPROPERTY(EditAnywhere, Category = "Pitch", DisplayName = "Display Frequency Multiplier Range", meta = (EditCondition = "MappingType == ETechAudioToolsFloatMappingType::Pitch && DisplayPitchUnits == ETechAudioToolsPitchUnit::FrequencyMultiplier", EditConditionHides, ClampMin = "0.125", ClampMax = "8"))
	FFloatInterval DisplayRange_FrequencyMultiplier { 0.5f, 2.f };

	// Returns the units used by the Source or Display endpoints.
	UFUNCTION(BlueprintPure, Category = "Range")
	TECHAUDIOTOOLS_API ETechAudioToolsFloatUnit GetUnits(ETechAudioToolsMappingEndpoint Endpoint) const;

	// Returns the range minimum of the source range.
	UFUNCTION(BlueprintPure, Category = "Range")
	float GetSourceMin() const { return GetRange(ETechAudioToolsMappingEndpoint::Source).Min; }

	// Returns the range maximum of the source range.
	UFUNCTION(BlueprintPure, Category = "Range")
	float GetSourceMax() const { return GetRange(ETechAudioToolsMappingEndpoint::Source).Max; }

	// Returns the range minimum of the display range.
	UFUNCTION(BlueprintPure, Category = "Range")
	float GetDisplayMin() const { return GetRange(ETechAudioToolsMappingEndpoint::Display).Min; }

	// Returns the range maximum of the display range.
	UFUNCTION(BlueprintPure, Category = "Range")
	float GetDisplayMax() const { return GetRange(ETechAudioToolsMappingEndpoint::Display).Max; }

	TECHAUDIOTOOLS_API float SourceToDisplay(float InSource) const;
	TECHAUDIOTOOLS_API float DisplayToSource(float InDisplay) const;

	TECHAUDIOTOOLS_API float NormalizedToSource(float InNormalized) const;
	TECHAUDIOTOOLS_API float SourceToNormalized(float InSource) const;

	TECHAUDIOTOOLS_API float NormalizedToDisplay(float InNormalized) const;
	TECHAUDIOTOOLS_API float DisplayToNormalized(float InDisplay) const;

private:
	TECHAUDIOTOOLS_API FFloatInterval GetRange(const ETechAudioToolsMappingEndpoint Endpoint) const;
};
