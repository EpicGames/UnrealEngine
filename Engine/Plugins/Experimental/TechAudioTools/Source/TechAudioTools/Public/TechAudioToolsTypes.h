// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "TechAudioToolsTypes.generated.h"

// Identifies which side of a mapping a value belongs to.
UENUM(BlueprintType)
enum class ETechAudioToolsMappingEndpoint : uint8
{
	Source				UMETA(DisplayName = "Source",				ToolTip = "Refers to values intended to be used by an internal system (e.g. a MetaSound"),
	Display				UMETA(DisplayName = "Display",				ToolTip = "Refers to values intended to be displayed via a user interface (e.g. a MetaSound preset widget"),
};

// Float unit types used primarily for formatting values.
UENUM(BlueprintType)
enum class ETechAudioToolsFloatUnit : uint8
{
	None				UMETA(DisplayName = "None"),
	Multiplier  		UMETA(DisplayName = "Multiplier",   		ToolTip = "Linear multiplier (1.0 = unity)"),
	Decibels			UMETA(DisplayName = "Decibels", 			ToolTip = "Amplitude in dB (0 dB = unity)"),
	Hertz   			UMETA(DisplayName = "Hertz",				ToolTip = "Frequency in Hertz (e.g. 20-20,000)"),
	Semitones   		UMETA(DisplayName = "Semitones",			ToolTip = "Pitch offset in semitones (-12 = one octave down)"),
	TimeSeconds 		UMETA(DisplayName = "Seconds",  			ToolTip = "Time in seconds"),
	TimeMilliseconds	UMETA(DisplayName = "Milliseconds", 		ToolTip = "Time in ms"),
	Degrees 			UMETA(DisplayName = "Degrees",  			ToolTip = "Angle in degrees (0-360)"),
	Pan 				UMETA(DisplayName = "Pan",  				ToolTip = "Pan value: negative = L, 0 = C, positive = R"),
	BandwidthOct		UMETA(DisplayName = "Bandwidth",			ToolTip = "Filter bandwidth in octaves (1.0 = one octave)"),
	Tempo   			UMETA(DisplayName = "Tempo",				ToolTip = "Tempo in beats per minute (BPM)")
};

// The mapping logic applied between source and display values.
UENUM(BlueprintType)
enum class ETechAudioToolsFloatMappingType : uint8
{
	Default 			UMETA(DisplayName = "Default",				ToolTip = "No mapping is applied. Display range is the same as the source range."),
	MapRange			UMETA(DisplayName = "Map Range",			ToolTip = "Linear remap between the source and display ranges. Does not convert units (e.g. seconds will not be converted to milliseconds)."),
	Volume				UMETA(DisplayName = "Volume",				ToolTip = "Remaps a given volume range and converts between Linear Gain and Decibels if necessary."),
	Pitch				UMETA(DisplayName = "Pitch",				ToolTip = "Remaps a given pitch range and converts between Semitones and Frequency Multiplier if necessary."),
};

// Defines the units used when interpreting a float as volume.
UENUM()
enum class ETechAudioToolsVolumeUnit : uint8
{
	Decibels			UMETA(DisplayName = "Decibels", 			ToolTip = "Decibels (logarithmic)"),
	LinearGain  		UMETA(DisplayName = "Linear Gain",   		ToolTip = "Linear gain multiplier"),
};

// Defines the units used when interpreting a float as pitch.
UENUM()
enum class ETechAudioToolsPitchUnit : uint8
{
	Semitones   		UMETA(DisplayName = "Semitones",			ToolTip = "Pitch offset in semitones"),
	FrequencyMultiplier UMETA(DisplayName = "Frequency Multiplier",	ToolTip = "Frequency multiplier (2.0 = one octave up)"),
};
