// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MVVMViewModelBase.h"
#include "Components/AudioComponent.h"

#include "AudioComponentViewModel.generated.h"

/**
 * Viewmodel for binding audio component state to widgets.
 */
UCLASS(MinimalAPI, BlueprintType, DisplayName = "Audio Component Viewmodel")
class UAudioComponentViewModel : public UMVVMViewModelBase
{
	GENERATED_BODY()

private:
	// Returns the enumerated play state of the audio component.
	UPROPERTY(BlueprintReadOnly, Transient, FieldNotify, Getter = "GetPlayState", Category = "Audio Component", meta = (AllowPrivateAccess))
	EAudioComponentPlayState PlayState = EAudioComponentPlayState::Stopped;

	// True if the audio component is playing.
	UPROPERTY(BlueprintReadOnly, Transient, FieldNotify, Getter = "IsPlaying", Category = "Audio Component", meta = (AllowPrivateAccess))
	bool bIsPlaying = false;

	// True if the audio component is stopped.
	UPROPERTY(BlueprintReadOnly, Transient, FieldNotify, Getter = "IsStopped", Category = "Audio Component", meta = (AllowPrivateAccess))
	bool bIsStopped = true;

	// True if the audio component is fading in.
	UPROPERTY(BlueprintReadOnly, Transient, FieldNotify, Getter = "IsFadingIn", Category = "Audio Component", meta = (AllowPrivateAccess))
	bool bIsFadingIn = false;

	// True if the audio component is fading out.
	UPROPERTY(BlueprintReadOnly, Transient, FieldNotify, Getter = "IsFadingOut", Category = "Audio Component", meta = (AllowPrivateAccess))
	bool bIsFadingOut = false;

	// True if the audio component is virtualized.
	UPROPERTY(BlueprintReadOnly, Transient, FieldNotify, Getter = "IsVirtualized", Category = "Audio Component", meta = (AllowPrivateAccess))
	bool bIsVirtualized = false;

protected:
	UPROPERTY(Transient)
	TWeakObjectPtr<UAudioComponent> AudioComponent;

public:
	// Sets the audio component this viewmodel should bind to.
	UFUNCTION(BlueprintCallable, Category = "TechAudioTools|Audio Component")
	TECHAUDIOTOOLS_API virtual void SetAudioComponent(UAudioComponent* InAudioComponent);

	EAudioComponentPlayState GetPlayState() const { return PlayState; }
	bool IsPlaying() const { return bIsPlaying; }
	bool IsStopped() const { return bIsStopped; }
	bool IsFadingIn() const { return bIsFadingIn; }
	bool IsFadingOut() const { return bIsFadingOut; }
	bool IsVirtualized() const { return bIsVirtualized; }

	virtual void BeginDestroy() override;

protected:
	void BindDelegates();
	void UnbindDelegates();

	UFUNCTION()
	void OnAudioFinished();

	UFUNCTION()
	void OnVirtualizationChanged(bool bInIsVirtualized);

private:
	UFUNCTION()
	void SetPlayState(EAudioComponentPlayState NewPlayState);
};
