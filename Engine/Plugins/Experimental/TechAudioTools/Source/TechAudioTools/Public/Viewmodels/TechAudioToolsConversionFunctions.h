// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Kismet/BlueprintFunctionLibrary.h"

#include "TechAudioToolsConversionFunctions.generated.h"

UCLASS(MinimalAPI)
class UTechAudioToolsConversionFunctions : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:
};
