// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class TechAudioTools : ModuleRules
{
    public TechAudioTools(ReadOnlyTargetRules Target) : base(Target)
    {
        PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

        PublicDependencyModuleNames.AddRange(
            new string[]
            {
                "Core",
                "ModelViewViewModel",
            }
        );

        PrivateDependencyModuleNames.AddRange(
            new string[]
            {
                "CoreUObject",
                "Engine",
            }
        );
    }
}
