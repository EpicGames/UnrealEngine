// Copyright Epic Games, Inc. All Rights Reserved.

#include "ViewModels/MetaSoundLiteralViewModel.h"

#include "TechAudioToolsFloatMapping.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MetaSoundLiteralViewModel)

void UMetaSoundLiteralViewModel_Boolean::PostInitProperties()
{
	Super::PostInitProperties();
	SetFlags(RF_Transactional);
}

void UMetaSoundLiteralViewModel_Boolean::SetLiteral(const FMetasoundFrontendLiteral& InLiteral)
{
	if (UE_MVVM_SET_PROPERTY_VALUE(Literal, InLiteral))
	{
		Literal = InLiteral;
		InLiteral.TryGet(SourceValue);
		UE_MVVM_BROADCAST_FIELD_VALUE_CHANGED(SourceValue);
	}
}

void UMetaSoundLiteralViewModel_Boolean::SetSourceValue(const bool InSourceValue)
{
	if (UE_MVVM_SET_PROPERTY_VALUE(SourceValue, InSourceValue))
	{
		Literal.Set(InSourceValue);
		UE_MVVM_BROADCAST_FIELD_VALUE_CHANGED(Literal);
	}
}

#if WITH_EDITOR
void UMetaSoundLiteralViewModel_Boolean::PostEditUndo()
{
	Super::PostEditUndo();
	SetSourceValue(SourceValue);
}
#endif

void UMetaSoundLiteralViewModel_BooleanArray::PostInitProperties()
{
	Super::PostInitProperties();
	SetFlags(RF_Transactional);
}

void UMetaSoundLiteralViewModel_BooleanArray::SetLiteral(const FMetasoundFrontendLiteral& InLiteral)
{
	if (UE_MVVM_SET_PROPERTY_VALUE(Literal, InLiteral))
	{
		Literal = InLiteral;
		InLiteral.TryGet(SourceValue);
		UE_MVVM_BROADCAST_FIELD_VALUE_CHANGED(SourceValue);
	}
}

void UMetaSoundLiteralViewModel_BooleanArray::SetSourceValue(const TArray<bool>& InSourceValue)
{
	if (UE_MVVM_SET_PROPERTY_VALUE(SourceValue, InSourceValue))
	{
		Literal.Set(InSourceValue);
		UE_MVVM_BROADCAST_FIELD_VALUE_CHANGED(Literal);
	}
}

#if WITH_EDITOR
void UMetaSoundLiteralViewModel_BooleanArray::PostEditUndo()
{
	Super::PostEditUndo();
	SetSourceValue(SourceValue);
}
#endif

void UMetaSoundLiteralViewModel_Integer::PostInitProperties()
{
	Super::PostInitProperties();
	SetFlags(RF_Transactional);
}

float UMetaSoundLiteralViewModel_Integer::GetStepSize() const
{
	if (!Range.HasLowerBound() || !Range.HasUpperBound())
	{
		return 1.f;
	}

	if (!Range.GetLowerBound().IsClosed() || !Range.GetUpperBound().IsClosed())
	{
		return 1.f;
	}

	int32 RangeMin = Range.GetLowerBoundValue();
	int32 RangeMax = Range.GetUpperBoundValue();

	if (Range.GetLowerBound().IsExclusive())
	{
		RangeMin += 1;
	}
	if (Range.GetUpperBound().IsExclusive())
	{
		RangeMax -= 1;
	}

	const int32 Span = RangeMax - RangeMin;
	if (Span <= 0)
	{
		return 1.f;
	}

	const float Step = 1.f / Span;
	return FMath::Clamp(Step, 0.f, 1.f);
}

void UMetaSoundLiteralViewModel_Integer::SetLiteral(const FMetasoundFrontendLiteral& InLiteral)
{
	const bool bInitializing = Literal.GetType() == EMetasoundFrontendLiteralType::None;

	if (!bIsUpdating)
	{
		bIsUpdating = true;
		Literal = InLiteral;
		InLiteral.TryGet(SourceValue);

		if (!Range.HasLowerBound() || !Range.HasUpperBound())
		{
			NormalizedValue = SourceValue;
		}
		else
		{
			const float RangeMin = Range.GetLowerBoundValue();
			const float RangeMax = Range.GetUpperBoundValue();
			NormalizedValue = FMath::GetRangePct(RangeMin, RangeMax, SourceValue);
		}

		// Don't broadcast Literal changed when setting the literal for the first time to avoid overriding inherited value
		if (!bInitializing)
		{
			UE_MVVM_BROADCAST_FIELD_VALUE_CHANGED(Literal);
		}
		UE_MVVM_BROADCAST_FIELD_VALUE_CHANGED(SourceValue);
		UE_MVVM_BROADCAST_FIELD_VALUE_CHANGED(NormalizedValue);
		bIsUpdating = false;
	}
}

void UMetaSoundLiteralViewModel_Integer::SetSourceValue(const int32 InSourceValue)
{
	if (!bIsUpdating)
	{
		bIsUpdating = true;
		SourceValue = InSourceValue;
		Literal.Set(SourceValue);

		if (!Range.HasLowerBound() || !Range.HasUpperBound())
		{
			NormalizedValue = SourceValue;
		}
		else
		{
			const float RangeMin = Range.GetLowerBoundValue();
			const float RangeMax = Range.GetUpperBoundValue();
			NormalizedValue = FMath::GetRangePct(RangeMin, RangeMax, SourceValue);
		}

		UE_MVVM_BROADCAST_FIELD_VALUE_CHANGED(Literal);
		UE_MVVM_BROADCAST_FIELD_VALUE_CHANGED(SourceValue);
		UE_MVVM_BROADCAST_FIELD_VALUE_CHANGED(NormalizedValue);
		bIsUpdating = false;
	}
}

void UMetaSoundLiteralViewModel_Integer::SetNormalizedValue(const float InNormalizedValue)
{
	if (!bIsUpdating)
	{
		bIsUpdating = true;
		NormalizedValue = InNormalizedValue;

		int32 RangeMin = 0;
		int32 RangeMax = 1;
		if (Range.HasLowerBound() && Range.HasUpperBound())
		{
			RangeMin = Range.GetLowerBoundValue();
			RangeMax = Range.GetUpperBoundValue();

			if (Range.GetLowerBound().IsExclusive())
			{
				RangeMin += 1;
			}
			if (Range.GetUpperBound().IsExclusive())
			{
				RangeMax -= 1;
			}
		}

		const FVector2D NormalizedRange(0, 1);
		const float UnmappedFloat = FMath::GetMappedRangeValueUnclamped(NormalizedRange, FVector2D(RangeMin, RangeMax), NormalizedValue);
		SourceValue = FMath::RoundToInt(UnmappedFloat);

		Literal.Set(SourceValue);

		UE_MVVM_BROADCAST_FIELD_VALUE_CHANGED(Literal);
		UE_MVVM_BROADCAST_FIELD_VALUE_CHANGED(SourceValue);
		UE_MVVM_BROADCAST_FIELD_VALUE_CHANGED(NormalizedValue);
		bIsUpdating = false;
	}
}

#if WITH_EDITOR
void UMetaSoundLiteralViewModel_Integer::PostEditUndo()
{
	Super::PostEditUndo();
	SetSourceValue(SourceValue);
}
#endif

void UMetaSoundLiteralViewModel_IntegerArray::PostInitProperties()
{
	Super::PostInitProperties();
	SetFlags(RF_Transactional);
}

void UMetaSoundLiteralViewModel_IntegerArray::SetLiteral(const FMetasoundFrontendLiteral& InLiteral)
{
	if (UE_MVVM_SET_PROPERTY_VALUE(Literal, InLiteral))
	{
		Literal = InLiteral;
		InLiteral.TryGet(SourceValue);
		UE_MVVM_BROADCAST_FIELD_VALUE_CHANGED(SourceValue);
	}
}

void UMetaSoundLiteralViewModel_IntegerArray::SetSourceValue(const TArray<int32>& InSourceValue)
{
	if (UE_MVVM_SET_PROPERTY_VALUE(SourceValue, InSourceValue))
	{
		Literal.Set(InSourceValue);
		UE_MVVM_BROADCAST_FIELD_VALUE_CHANGED(Literal);
	}
}

#if WITH_EDITOR
void UMetaSoundLiteralViewModel_IntegerArray::PostEditUndo()
{
	Super::PostEditUndo();
	SetSourceValue(SourceValue);
}
#endif

float UMetaSoundLiteralViewModel_Float::GetSourceRangeMin() const
{
	return RangeValues ? RangeValues->GetSourceMin() : 0.f;
}

float UMetaSoundLiteralViewModel_Float::GetSourceRangeMax() const
{
	return RangeValues ? RangeValues->GetSourceMax() : 0.f;
}

ETechAudioToolsFloatUnit UMetaSoundLiteralViewModel_Float::GetSourceUnits() const
{
	return RangeValues ? RangeValues->GetUnits(ETechAudioToolsMappingEndpoint::Source) : ETechAudioToolsFloatUnit::None;
}

float UMetaSoundLiteralViewModel_Float::GetDisplayRangeMin() const
{
	return RangeValues ? RangeValues->GetDisplayMin() : 0.f;
}

float UMetaSoundLiteralViewModel_Float::GetDisplayRangeMax() const
{
	return RangeValues ? RangeValues->GetDisplayMax() : 0.f;
}

ETechAudioToolsFloatUnit UMetaSoundLiteralViewModel_Float::GetDisplayUnits() const
{
	return RangeValues ? RangeValues->GetUnits(ETechAudioToolsMappingEndpoint::Display) : ETechAudioToolsFloatUnit::None;
}

UMetaSoundLiteralViewModel_Float::UMetaSoundLiteralViewModel_Float(const FObjectInitializer& ObjectInitializer)
{
	RangeValues = ObjectInitializer.CreateDefaultSubobject<UTechAudioToolsFloatMapping>(this, TEXT("RangeValues"));
}

void UMetaSoundLiteralViewModel_Float::PostInitProperties()
{
	Super::PostInitProperties();
	SetFlags(RF_Transactional);
}

void UMetaSoundLiteralViewModel_Float::SetShowDisplayValues(const bool bInIsShowingDisplayValues)
{
	if (!bIsUpdating && RangeValues)
	{
		bIsUpdating = true;
		bShowDisplayValues = bInIsShowingDisplayValues;
		NormalizedValue = bShowDisplayValues ? RangeValues->DisplayToNormalized(DisplayValue) : RangeValues->SourceToNormalized(SourceValue);
		UE_MVVM_BROADCAST_FIELD_VALUE_CHANGED(NormalizedValue);
		UE_MVVM_BROADCAST_FIELD_VALUE_CHANGED(bShowDisplayValues);
		bIsUpdating = false;
	}
}

void UMetaSoundLiteralViewModel_Float::SetLiteral(const FMetasoundFrontendLiteral& InLiteral)
{
	const bool bInitializing = Literal.GetType() == EMetasoundFrontendLiteralType::None;

	if (!bIsUpdating && RangeValues)
	{
		bIsUpdating = true;
		Literal = InLiteral;
		InLiteral.TryGet(SourceValue);
		DisplayValue = RangeValues->SourceToDisplay(SourceValue);
		NormalizedValue = bShowDisplayValues ? RangeValues->DisplayToNormalized(DisplayValue) : RangeValues->SourceToNormalized(SourceValue);

		// Don't broadcast Literal changed when setting the literal for the first time to avoid overriding inherited value
		if (!bInitializing)
		{
			UE_MVVM_BROADCAST_FIELD_VALUE_CHANGED(Literal);
		}
		UE_MVVM_BROADCAST_FIELD_VALUE_CHANGED(SourceValue);
		UE_MVVM_BROADCAST_FIELD_VALUE_CHANGED(NormalizedValue);
		UE_MVVM_BROADCAST_FIELD_VALUE_CHANGED(DisplayValue);
		bIsUpdating = false;
	}
}

void UMetaSoundLiteralViewModel_Float::SetSourceValue(const float InSourceValue)
{
	if (!bIsUpdating && RangeValues)
	{
		bIsUpdating = true;
		SourceValue = InSourceValue;
		DisplayValue = RangeValues->SourceToDisplay(SourceValue);
		NormalizedValue = bShowDisplayValues ? RangeValues->DisplayToNormalized(DisplayValue) : RangeValues->SourceToNormalized(SourceValue);
		Literal.Set(SourceValue);

		UE_MVVM_BROADCAST_FIELD_VALUE_CHANGED(Literal);
		UE_MVVM_BROADCAST_FIELD_VALUE_CHANGED(SourceValue);
		UE_MVVM_BROADCAST_FIELD_VALUE_CHANGED(NormalizedValue);
		UE_MVVM_BROADCAST_FIELD_VALUE_CHANGED(DisplayValue);
		bIsUpdating = false;
	}
}

void UMetaSoundLiteralViewModel_Float::SetNormalizedValue(const float InNormalizedValue)
{
	if (!bIsUpdating && RangeValues)
	{
		bIsUpdating = true;
		NormalizedValue = InNormalizedValue;

		// Properly ensures the normalized range is mapped correctly
		if (bShowDisplayValues)
		{
			DisplayValue = RangeValues->NormalizedToDisplay(InNormalizedValue);
			SourceValue = RangeValues->DisplayToSource(DisplayValue);
		}
		else
		{
			SourceValue = RangeValues->NormalizedToSource(InNormalizedValue);
			DisplayValue = RangeValues->SourceToDisplay(SourceValue);
		}

		Literal.Set(SourceValue);

		UE_MVVM_BROADCAST_FIELD_VALUE_CHANGED(Literal);
		UE_MVVM_BROADCAST_FIELD_VALUE_CHANGED(SourceValue);
		UE_MVVM_BROADCAST_FIELD_VALUE_CHANGED(NormalizedValue);
		UE_MVVM_BROADCAST_FIELD_VALUE_CHANGED(DisplayValue);
		bIsUpdating = false;
	}
}

void UMetaSoundLiteralViewModel_Float::SetDisplayValue(const float InDisplayValue)
{
	if (!bIsUpdating && RangeValues)
	{
		bIsUpdating = true;
		DisplayValue = InDisplayValue;
		SourceValue = RangeValues->DisplayToSource(InDisplayValue);
		NormalizedValue = bShowDisplayValues ? RangeValues->DisplayToNormalized(DisplayValue) : RangeValues->SourceToNormalized(SourceValue);
		Literal.Set(SourceValue);

		UE_MVVM_BROADCAST_FIELD_VALUE_CHANGED(Literal);
		UE_MVVM_BROADCAST_FIELD_VALUE_CHANGED(SourceValue);
		UE_MVVM_BROADCAST_FIELD_VALUE_CHANGED(NormalizedValue);
		UE_MVVM_BROADCAST_FIELD_VALUE_CHANGED(DisplayValue);
		bIsUpdating = false;
	}
}

#if WITH_EDITOR
void UMetaSoundLiteralViewModel_Float::PostEditUndo()
{
	Super::PostEditUndo();
	SetSourceValue(SourceValue);
}
#endif

void UMetaSoundLiteralViewModel_FloatArray::PostInitProperties()
{
	Super::PostInitProperties();
	SetFlags(RF_Transactional);
}

void UMetaSoundLiteralViewModel_FloatArray::SetLiteral(const FMetasoundFrontendLiteral& InLiteral)
{
	if (UE_MVVM_SET_PROPERTY_VALUE(Literal, InLiteral))
	{
		Literal = InLiteral;
		InLiteral.TryGet(SourceValue);
		UE_MVVM_BROADCAST_FIELD_VALUE_CHANGED(SourceValue);
	}
}

void UMetaSoundLiteralViewModel_FloatArray::SetSourceValue(const TArray<float>& InSourceValue)
{
	if (UE_MVVM_SET_PROPERTY_VALUE(SourceValue, InSourceValue))
	{
		Literal.Set(InSourceValue);
		UE_MVVM_BROADCAST_FIELD_VALUE_CHANGED(Literal);
	}
}

#if WITH_EDITOR
void UMetaSoundLiteralViewModel_FloatArray::PostEditUndo()
{
	Super::PostEditUndo();
	SetSourceValue(SourceValue);
}
#endif

void UMetaSoundLiteralViewModel_String::PostInitProperties()
{
	Super::PostInitProperties();
	SetFlags(RF_Transactional);
}

void UMetaSoundLiteralViewModel_String::SetLiteral(const FMetasoundFrontendLiteral& InLiteral)
{
	if (UE_MVVM_SET_PROPERTY_VALUE(Literal, InLiteral))
	{
		Literal = InLiteral;
		InLiteral.TryGet(SourceValue);
		UE_MVVM_BROADCAST_FIELD_VALUE_CHANGED(SourceValue);
	}
}

void UMetaSoundLiteralViewModel_String::SetSourceValue(const FString& InSourceValue)
{
	if (UE_MVVM_SET_PROPERTY_VALUE(SourceValue, InSourceValue))
	{
		Literal.Set(InSourceValue);
		UE_MVVM_BROADCAST_FIELD_VALUE_CHANGED(Literal);
	}
}

#if WITH_EDITOR
void UMetaSoundLiteralViewModel_String::PostEditUndo()
{
	Super::PostEditUndo();
	SetSourceValue(SourceValue);
}
#endif

void UMetaSoundLiteralViewModel_StringArray::PostInitProperties()
{
	Super::PostInitProperties();
	SetFlags(RF_Transactional);
}

void UMetaSoundLiteralViewModel_StringArray::SetLiteral(const FMetasoundFrontendLiteral& InLiteral)
{
	if (UE_MVVM_SET_PROPERTY_VALUE(Literal, InLiteral))
	{
		Literal = InLiteral;
		InLiteral.TryGet(SourceValue);
		UE_MVVM_BROADCAST_FIELD_VALUE_CHANGED(SourceValue);
	}
}

void UMetaSoundLiteralViewModel_StringArray::SetSourceValue(const TArray<FString>& InSourceValue)
{
	if (UE_MVVM_SET_PROPERTY_VALUE(SourceValue, InSourceValue))
	{
		Literal.Set(InSourceValue);
		UE_MVVM_BROADCAST_FIELD_VALUE_CHANGED(Literal);
	}
}

#if WITH_EDITOR
void UMetaSoundLiteralViewModel_StringArray::PostEditUndo()
{
	Super::PostEditUndo();
	SetSourceValue(SourceValue);
}
#endif

void UMetaSoundLiteralViewModel_Object::PostInitProperties()
{
	Super::PostInitProperties();
	SetFlags(RF_Transactional);
}

void UMetaSoundLiteralViewModel_Object::SetLiteral(const FMetasoundFrontendLiteral& InLiteral)
{
	if (UE_MVVM_SET_PROPERTY_VALUE(Literal, InLiteral))
	{
		Literal = InLiteral;
		UObject* SourceValuePtr = nullptr;
		InLiteral.TryGet(SourceValuePtr);
		SourceValue = SourceValuePtr;
		UE_MVVM_BROADCAST_FIELD_VALUE_CHANGED(SourceValue);
	}
}

void UMetaSoundLiteralViewModel_Object::SetSourceValue(UObject* InSourceValue)
{
	if (UE_MVVM_SET_PROPERTY_VALUE(SourceValue, InSourceValue))
	{
		Literal.Set(InSourceValue);
		UE_MVVM_BROADCAST_FIELD_VALUE_CHANGED(Literal);
	}
}

#if WITH_EDITOR
void UMetaSoundLiteralViewModel_Object::PostEditUndo()
{
	Super::PostEditUndo();
	SetSourceValue(SourceValue);
}
#endif

void UMetaSoundLiteralViewModel_ObjectArray::PostInitProperties()
{
	Super::PostInitProperties();
	SetFlags(RF_Transactional);
}

void UMetaSoundLiteralViewModel_ObjectArray::SetLiteral(const FMetasoundFrontendLiteral& InLiteral)
{
	if (UE_MVVM_SET_PROPERTY_VALUE(Literal, InLiteral))
	{
		Literal = InLiteral;
		TArray<UObject*> SourceValuePtr;
		InLiteral.TryGet(SourceValuePtr);
		SourceValue = SourceValuePtr;
		UE_MVVM_BROADCAST_FIELD_VALUE_CHANGED(SourceValue);
	}
}

void UMetaSoundLiteralViewModel_ObjectArray::SetSourceValue(const TArray<TObjectPtr<UObject>>& InSourceValue)
{
	if (UE_MVVM_SET_PROPERTY_VALUE(SourceValue, InSourceValue))
	{
		Literal.Set(InSourceValue);
		UE_MVVM_BROADCAST_FIELD_VALUE_CHANGED(Literal);
	}
}

#if WITH_EDITOR
void UMetaSoundLiteralViewModel_ObjectArray::PostEditUndo()
{
	Super::PostEditUndo();
	SetSourceValue(SourceValue);
}
#endif
