// Copyright Epic Games, Inc. All Rights Reserved.

#include "ViewModels/MetaSoundViewModel.h"

#include "DocumentTemplates/MetasoundFrontendPresetTemplate.h"
#include "Logging/StructuredLog.h"
#include "MetasoundDocumentBuilderRegistry.h"
#include "MetasoundFrontendPages.h"
#include "TechAudioToolsLog.h"
#include "ViewModels/MetaSoundViewModelConversionFunctions.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MetaSoundViewModel)

TArray<UMetaSoundInputViewModel*> UMetaSoundViewModel::GetInputViewModels() const
{
	TArray<TObjectPtr<UMetaSoundInputViewModel>> InputViewModelsArray;
	InputViewModels.GenerateValueArray(InputViewModelsArray);
	return InputViewModelsArray;
}

UMetaSoundInputViewModel* UMetaSoundViewModel::FindInputViewModel(const FName InputViewModelName) const
{
	return InputViewModels.FindRef(InputViewModelName);
}

TArray<UMetaSoundOutputViewModel*> UMetaSoundViewModel::GetOutputViewModels() const
{
	TArray<TObjectPtr<UMetaSoundOutputViewModel>> OutputViewModelsArray;
	OutputViewModels.GenerateValueArray(OutputViewModelsArray);
	return OutputViewModelsArray;
}

UMetaSoundOutputViewModel* UMetaSoundViewModel::FindOutputViewModel(const FName OutputViewModelName) const
{
	return OutputViewModels.FindRef(OutputViewModelName);
}

void UMetaSoundViewModel::InitializeMetaSound(const TScriptInterface<IMetaSoundDocumentInterface> InMetaSound)
{
	if (!InMetaSound)
	{
		Reset();
		return;
	}

	Builder = Metasound::Engine::FDocumentBuilderRegistry::GetChecked().FindBuilderObject(InMetaSound);
	Initialize(Builder);
}

void UMetaSoundViewModel::Initialize(UMetaSoundBuilderBase* InBuilder)
{
	using namespace Metasound::Frontend;

	Reset();

	if (!InBuilder)
	{
		UE_LOGFMT(LogTechAudioTools, Warning, "Unable to initialize MetaSoundViewModel. Builder was null.");
		return;
	}

	Builder = InBuilder;

	const FMetaSoundFrontendDocumentBuilder& DocumentBuilder = InBuilder->GetConstBuilder();
	UE_MVVM_SET_PROPERTY_VALUE(bIsPreset, DocumentBuilder.IsDocumentConfiguredBy<FPresetTemplate>());

	CreateMemberViewModels();
	SetIsInitialized(true);
}

void UMetaSoundViewModel::Reset()
{
	SetIsInitialized(false);
	UE_MVVM_SET_PROPERTY_VALUE(bIsPreset, false);
	Builder = nullptr;

	InputViewModels.Empty();
	UE_MVVM_BROADCAST_FIELD_VALUE_CHANGED(GetInputViewModels);

	OutputViewModels.Empty();
	UE_MVVM_BROADCAST_FIELD_VALUE_CHANGED(GetOutputViewModels);
}

void UMetaSoundViewModel::SetIsInitialized(const bool bInIsInitialized)
{
	UE_MVVM_SET_PROPERTY_VALUE(bIsInitialized, bInIsInitialized);
}

void UMetaSoundViewModel::SetInputName(const FName& OldName, const FName& NewName) const
{
	if (OldName == NewName)
	{
		return;
	}

	if (InputViewModels.Contains(NewName))
	{
		UE_LOGFMT(LogTechAudioTools, Warning, "Cannot set input name '{NewName}'. An input already exists with that name.", NewName);
		return;
	}

	if (UMetaSoundViewModelConversionFunctions::IsInterfaceMember(NewName))
	{
		UE_LOGFMT(LogTechAudioTools, Warning, "Cannot set input name '{NewName}'. Name already belongs to an interface.", NewName);
		return;
	}

	if (Builder)
	{
		EMetaSoundBuilderResult Result;
		Builder->SetGraphInputName(OldName, NewName, Result);
	}
}

void UMetaSoundViewModel::SetInputDataType(const FName& InputName, const FName& DataType) const
{
	if (!Metasound::Frontend::IDataTypeRegistry::Get().IsRegistered(DataType))
	{
		UE_LOGFMT(LogTechAudioTools, Warning, "Failed to set data type for {InputName}. '{DataType}' is not a registered data type.", InputName, DataType);
		return;
	}

	if (Builder)
	{
		FMetaSoundFrontendDocumentBuilder& DocumentBuilder = Builder->GetBuilder();
		DocumentBuilder.SetGraphInputDataType(InputName, DataType);
	}
}

void UMetaSoundViewModel::SetInputDefaultLiteral(const FName& InputName, const FMetasoundFrontendLiteral& DefaultLiteral) const
{
	if (Builder)
	{
		EMetaSoundBuilderResult Result;
		Builder->SetGraphInputDefault(InputName, DefaultLiteral, Result);
	}
}

void UMetaSoundViewModel::SetInputOverridesDefault(const FName& InputName, const bool bOverridesDefault) const
{
	using namespace Metasound::Frontend;

	if (Builder)
	{
		FMetaSoundFrontendDocumentBuilder& DocumentBuilder = Builder->GetBuilder();

		if (const FMetasoundFrontendClassInput* Input = DocumentBuilder.FindGraphInput(InputName))
		{
			if (FPresetVertexMetadata* PresetMetadata = DocumentBuilder.FindVertexMetadata<FPresetVertexMetadata>(Input->NodeID))
			{
				PresetMetadata->bOverrideInheritedDefault = bOverridesDefault;
			}
		}
	}
}

void UMetaSoundViewModel::SetInputIsConstructorPin(const FName& InputName, const bool bIsConstructorPin) const
{
	if (const UMetaSoundInputViewModel* InputViewModel = InputViewModels.FindRef(InputName))
	{
		const FName DataType = InputViewModel->GetDataType();
		if (!UMetaSoundViewModelConversionFunctions::IsConstructorType(DataType))
		{
			UE_LOGFMT(LogTechAudioTools, Warning, "Failed to set bIsConstructorPin on {InputName}. Data type '{DataType}' cannot be a constructor pin.", InputName, DataType);
			return;
		}
	}

	if (Builder)
	{
		FMetaSoundFrontendDocumentBuilder& DocumentBuilder = Builder->GetBuilder();
		const EMetasoundFrontendVertexAccessType AccessType = bIsConstructorPin ? EMetasoundFrontendVertexAccessType::Value : EMetasoundFrontendVertexAccessType::Reference;
		DocumentBuilder.SetGraphInputAccessType(InputName, AccessType);
	}
}

void UMetaSoundViewModel::SetOutputName(const FName& OldName, const FName& NewName) const
{
	if (OldName == NewName)
	{
		return;
	}

	if (OutputViewModels.Contains(NewName))
	{
		UE_LOGFMT(LogTechAudioTools, Warning, "Cannot set output name '{NewName}'. An output already exists with that name.", NewName);
		return;
	}

	if (UMetaSoundViewModelConversionFunctions::IsInterfaceMember(NewName))
	{
		UE_LOGFMT(LogTechAudioTools, Warning, "Cannot set output name '{NewName}'. Name already belongs to an interface.", NewName);
		return;
	}

	if (Builder)
	{
		EMetaSoundBuilderResult Result;
		Builder->SetGraphOutputName(OldName, NewName, Result);
	}
}

void UMetaSoundViewModel::SetOutputDataType(const FName& OutputName, const FName& DataType) const
{
	if (!Metasound::Frontend::IDataTypeRegistry::Get().IsRegistered(DataType))
	{
		UE_LOGFMT(LogTechAudioTools, Warning, "Failed to set data type for {OutputName}. Data type '{DataType}' is not a registered data type.", OutputName, DataType);
		return;
	}

	if (Builder)
	{
		FMetaSoundFrontendDocumentBuilder& DocumentBuilder = Builder->GetBuilder();
		DocumentBuilder.SetGraphOutputDataType(OutputName, DataType);
	}
}

void UMetaSoundViewModel::SetOutputIsConstructorPin(const FName& OutputName, const bool bIsConstructorPin) const
{
	if (const UMetaSoundOutputViewModel* OutputViewModel = OutputViewModels.FindRef(OutputName))
	{
		const FName DataType = OutputViewModel->GetDataType();
		if (!UMetaSoundViewModelConversionFunctions::IsConstructorType(DataType))
		{
			UE_LOGFMT(LogTechAudioTools, Warning, "Failed to set bIsConstructorPin on {OutputName}. Data type '{DataType}' cannot be a constructor pin.", OutputName, DataType);
			return;
		}
	}

	if (Builder)
	{
		FMetaSoundFrontendDocumentBuilder& DocumentBuilder = Builder->GetBuilder();
		const EMetasoundFrontendVertexAccessType AccessType = bIsConstructorPin ? EMetasoundFrontendVertexAccessType::Value : EMetasoundFrontendVertexAccessType::Reference;
		DocumentBuilder.SetGraphOutputAccessType(OutputName, AccessType);
	}
}

void UMetaSoundViewModel::OnInputAdded(const FName VertexName, FName DataType)
{
	if (Builder && !InputViewModels.Contains(VertexName))
	{
		const FMetaSoundFrontendDocumentBuilder& DocumentBuilder = Builder->GetConstBuilder();
		const FMetasoundFrontendDocument& FrontendDocument = DocumentBuilder.GetConstDocumentChecked();
		for (const FMetasoundFrontendClassInput& Input : FrontendDocument.RootGraph.GetDefaultInterface().Inputs)
		{
			if (Input.Name == VertexName)
			{
				if (UMetaSoundInputViewModel* InputViewModel = CreateInputViewModel(Input))
				{
					InputViewModel->SetIsInitialized(true);
					UE_MVVM_BROADCAST_FIELD_VALUE_CHANGED(GetInputViewModels);
				}
				return;
			}
		}
	}
}

void UMetaSoundViewModel::OnInputRemoved(const FName VertexName, FName DataType)
{
	if (InputViewModels.Remove(VertexName))
	{
		UE_MVVM_BROADCAST_FIELD_VALUE_CHANGED(GetInputViewModels);
	}
}

void UMetaSoundViewModel::OnInputNameChanged(const FName OldName, const FName NewName)
{
	if (UMetaSoundInputViewModel* InputViewModel = Cast<UMetaSoundInputViewModel>(InputViewModels.FindAndRemoveChecked(OldName)))
	{
		InputViewModels.Emplace(NewName, MoveTemp(InputViewModel));
		InputViewModel->OnInputNameChanged(NewName);
	}
}

void UMetaSoundViewModel::OnInputDataTypeChanged(const FName VertexName, const FName DataType)
{
	if (UMetaSoundInputViewModel* InputViewModel = InputViewModels.FindRef(VertexName))
	{
		InputViewModel->OnInputDataTypeChanged(DataType);
	}
}

void UMetaSoundViewModel::OnInputDefaultChanged(const FName VertexName, FMetasoundFrontendLiteral LiteralValue, const FName PageName, EMetaSoundLiteralChangeType ChangeType)
{
	if (UMetaSoundInputViewModel* InputViewModel = InputViewModels.FindRef(VertexName))
	{
		InputViewModel->OnInputDefaultChanged(LiteralValue, PageName, ChangeType);
	}
}

void UMetaSoundViewModel::OnInputInheritsDefaultChanged(const FName VertexName, const bool bInheritsDefault)
{
	if (UMetaSoundInputViewModel* InputViewModel = InputViewModels.FindRef(VertexName))
	{
		InputViewModel->OnInputInheritsDefaultChanged(bInheritsDefault);
	}
}

void UMetaSoundViewModel::OnInputIsConstructorPinChanged(const FName VertexName, const bool bIsConstructorPin)
{
	if (UMetaSoundInputViewModel* InputViewModel = InputViewModels.FindRef(VertexName))
	{
		InputViewModel->OnInputIsConstructorPinChanged(bIsConstructorPin);
	}
}

void UMetaSoundViewModel::OnOutputAdded(const FName VertexName, FName DataType)
{
	if (Builder && !OutputViewModels.Contains(VertexName))
	{
		const FMetaSoundFrontendDocumentBuilder& DocumentBuilder = Builder->GetConstBuilder();
		const FMetasoundFrontendDocument& FrontendDocument = DocumentBuilder.GetConstDocumentChecked();
		for (const FMetasoundFrontendClassOutput& Output : FrontendDocument.RootGraph.GetDefaultInterface().Outputs)
		{
			if (Output.Name == VertexName)
			{
				if (UMetaSoundOutputViewModel* OutputViewModel = CreateOutputViewModel(Output))
				{
					OutputViewModel->SetIsInitialized(true);
					UE_MVVM_BROADCAST_FIELD_VALUE_CHANGED(GetOutputViewModels);
				}
				return;
			}
		}
	}
}

void UMetaSoundViewModel::OnOutputRemoved(const FName VertexName, FName DataType)
{
	if (OutputViewModels.Remove(VertexName))
	{
		UE_MVVM_BROADCAST_FIELD_VALUE_CHANGED(GetOutputViewModels);
	}
}

void UMetaSoundViewModel::OnOutputNameChanged(const FName OldName, const FName NewName)
{
	if (UMetaSoundOutputViewModel* OutputViewModel = OutputViewModels.FindAndRemoveChecked(OldName))
	{
		OutputViewModels.Emplace(NewName, MoveTemp(OutputViewModel));
		OutputViewModel->OnOutputNameChanged(NewName);
	}
}

void UMetaSoundViewModel::OnOutputDataTypeChanged(const FName VertexName, const FName DataType)
{
	if (UMetaSoundOutputViewModel* OutputViewModel = OutputViewModels.FindRef(VertexName))
	{
		OutputViewModel->OnOutputDataTypeChanged(DataType);
	}
}

void UMetaSoundViewModel::OnOutputIsConstructorPinChanged(const FName VertexName, const bool bIsConstructorPin)
{
	if (UMetaSoundOutputViewModel* OutputViewModel = OutputViewModels.FindRef(VertexName))
	{
		OutputViewModel->OnOutputIsConstructorPinChanged(bIsConstructorPin);
	}
}

void UMetaSoundViewModel::CreateMemberViewModels()
{
	if (!Builder)
	{
		UE_LOGFMT(LogTechAudioTools, Warning, "Failed to create member viewmodels. Builder was null.");
		return;
	}

	const FMetaSoundFrontendDocumentBuilder& DocumentBuilder = Builder->GetConstBuilder();
	const FMetasoundFrontendDocument& FrontendDocument = DocumentBuilder.GetConstDocumentChecked();

	for (const FMetasoundFrontendClassInput& Input : FrontendDocument.RootGraph.GetDefaultInterface().Inputs)
	{
		if (UMetaSoundInputViewModel* InputViewModel = CreateInputViewModel(Input))
		{
			InputViewModel->SetIsInitialized(true);
		}
	}
	UE_MVVM_BROADCAST_FIELD_VALUE_CHANGED(GetInputViewModels);

	for (const FMetasoundFrontendClassOutput& Output : FrontendDocument.RootGraph.GetDefaultInterface().Outputs)
	{
		if (UMetaSoundOutputViewModel* OutputViewModel = CreateOutputViewModel(Output))
		{
			OutputViewModel->SetIsInitialized(true);
		}
	}
	UE_MVVM_BROADCAST_FIELD_VALUE_CHANGED(GetOutputViewModels);
}

UMetaSoundInputViewModel* UMetaSoundViewModel::CreateInputViewModel(const FMetasoundFrontendClassInput& InInput)
{
	using namespace Metasound::Frontend;

	if (UMetaSoundInputViewModel* InputViewModel = NewObject<UMetaSoundInputViewModel>(this, GetInputViewModelClass()))
	{
		FName PageName;
		FMetasoundFrontendLiteral DefaultLiteral;
		if (const FMetasoundFrontendClassInputDefault* Default = Metasound::Frontend::FindPreferredPage(InInput.GetDefaults(), UMetaSoundSettings::GetPageOrder()))
		{
			if (const UMetaSoundSettings* Settings = GetDefault<UMetaSoundSettings>())
			{
				if (const FMetaSoundPageSettings* PageSettings = Settings->FindPageSettings(Default->PageID))
				{
					PageName = PageSettings->Name;
				}
			}

			DefaultLiteral = Default->Literal;
		}

		bool bInheritsDefault = false;
		if (Builder)
		{
			const FMetaSoundFrontendDocumentBuilder& DocumentBuilder = Builder->GetConstBuilder();
			bInheritsDefault = false;
			if (const FPresetVertexMetadata* VertexMetadata = DocumentBuilder.FindConstVertexMetadata<FPresetVertexMetadata>(InInput.NodeID))
			{
				bInheritsDefault = !VertexMetadata->bOverrideInheritedDefault;
			}
		}

		InputViewModel->OnInputNameChanged(InInput.Name);
		InputViewModel->OnInputDataTypeChanged(InInput.TypeName);
		InputViewModel->OnInputDefaultChanged(DefaultLiteral, PageName);
		InputViewModel->OnInputInheritsDefaultChanged(bInheritsDefault);
		InputViewModel->OnInputIsConstructorPinChanged(InInput.AccessType == EMetasoundFrontendVertexAccessType::Value);

		return InputViewModels.Emplace(InInput.Name, InputViewModel);
	}

	return nullptr;
}

UMetaSoundOutputViewModel* UMetaSoundViewModel::CreateOutputViewModel(const FMetasoundFrontendClassOutput& InOutput)
{
	if (UMetaSoundOutputViewModel* OutputViewModel = NewObject<UMetaSoundOutputViewModel>(this, GetOutputViewModelClass()))
	{
		OutputViewModel->OnOutputNameChanged(InOutput.Name);
		OutputViewModel->OnOutputDataTypeChanged(InOutput.TypeName);
		OutputViewModel->OnOutputIsConstructorPinChanged(InOutput.AccessType == EMetasoundFrontendVertexAccessType::Value);

		return OutputViewModels.Emplace(InOutput.Name, OutputViewModel);
	}

	return nullptr;
}

TSubclassOf<UMetaSoundInputViewModel> UMetaSoundViewModel::GetInputViewModelClass() const
{
	return UMetaSoundInputViewModel::StaticClass();
}

TSubclassOf<UMetaSoundOutputViewModel> UMetaSoundViewModel::GetOutputViewModelClass() const
{
	return UMetaSoundOutputViewModel::StaticClass();
}

void UMetaSoundInputViewModel::SetIsInitialized(const bool bInIsInitialized)
{
	UE_MVVM_SET_PROPERTY_VALUE(bIsInitialized, bInIsInitialized);
}

void UMetaSoundInputViewModel::SetInputName(const FName& NewName)
{
	const FName OldName = InputName;
	if (const UMetaSoundViewModel* MetaSoundViewModel = Cast<UMetaSoundViewModel>(GetOuter()))
	{
		MetaSoundViewModel->SetInputName(OldName, NewName);
	}
}

void UMetaSoundInputViewModel::SetDataType(const FName& InDataType)
{
	if (!Metasound::Frontend::IDataTypeRegistry::Get().IsRegistered(InDataType))
	{
		UE_MVVM_BROADCAST_FIELD_VALUE_CHANGED(DataType);
		UE_LOGFMT(LogTechAudioTools, Warning, "Failed to set data type for {InputName}. '{InDataType}' is not a registered data type.", InputName, InDataType);
		return;
	}

	if (const UMetaSoundViewModel* MetaSoundViewModel = Cast<UMetaSoundViewModel>(GetOuter()))
	{
		MetaSoundViewModel->SetInputDataType(InputName, InDataType);
	}
}

void UMetaSoundInputViewModel::SetIsArray(const bool bInIsArray)
{
	if (UMetaSoundViewModelConversionFunctions::IsArrayType(DataType))
	{
		SetDataType(TechAudioTools::MetaSound::GetAdjustedDataType(DataType, bInIsArray));
	}
}

void UMetaSoundInputViewModel::SetIsConstructorPin(const bool bInIsConstructorPin)
{
	if (!UMetaSoundViewModelConversionFunctions::IsConstructorType(DataType))
	{
		UE_MVVM_BROADCAST_FIELD_VALUE_CHANGED(bIsConstructorPin);
		UE_LOGFMT(LogTechAudioTools, Warning, "Failed to set bIsConstructorPin on {InputName}. Data type '{DataType}' cannot be a constructor pin.", InputName, DataType);
		return;
	}

	if (const UMetaSoundViewModel* MetaSoundViewModel = Cast<UMetaSoundViewModel>(GetOuter()))
	{
		MetaSoundViewModel->SetInputIsConstructorPin(InputName, bInIsConstructorPin);
	}
}

void UMetaSoundInputViewModel::SetLiteral(const FMetasoundFrontendLiteral& InLiteral)
{
	if (const UMetaSoundViewModel* MetaSoundViewModel = Cast<UMetaSoundViewModel>(GetOuter()))
	{
		MetaSoundViewModel->SetInputDefaultLiteral(InputName, InLiteral);
	}
}

void UMetaSoundInputViewModel::SetOverridesDefault(const bool bInOverridesDefaultValue)
{
	if (const UMetaSoundViewModel* MetaSoundViewModel = Cast<UMetaSoundViewModel>(GetOuter()))
	{
		MetaSoundViewModel->SetInputOverridesDefault(InputName, bInOverridesDefaultValue);
	}
}

void UMetaSoundInputViewModel::OnInputNameChanged(const FName& NewName)
{
	UE_MVVM_SET_PROPERTY_VALUE(InputName, NewName);
}

void UMetaSoundInputViewModel::OnInputDataTypeChanged(const FName& NewDataType)
{
	if (UE_MVVM_SET_PROPERTY_VALUE(DataType, NewDataType))
	{
		UE_MVVM_BROADCAST_FIELD_VALUE_CHANGED(LiteralType);
		UE_MVVM_BROADCAST_FIELD_VALUE_CHANGED(bIsArray);
	}
}

void UMetaSoundInputViewModel::OnInputDefaultChanged(const FMetasoundFrontendLiteral& LiteralValue, const FName& PageName, EMetaSoundLiteralChangeType ChangeType)
{
	UE_MVVM_SET_PROPERTY_VALUE(Literal, LiteralValue);
}

void UMetaSoundInputViewModel::OnInputInheritsDefaultChanged(const bool bInheritsDefault)
{
	UE_MVVM_SET_PROPERTY_VALUE(bOverridesDefault, !bInheritsDefault);
}

void UMetaSoundInputViewModel::OnInputIsConstructorPinChanged(const bool bNewIsConstructorPin)
{
	UE_MVVM_SET_PROPERTY_VALUE(bIsConstructorPin, bNewIsConstructorPin);
}

void UMetaSoundOutputViewModel::SetIsInitialized(const bool bInIsInitialized)
{
	UE_MVVM_SET_PROPERTY_VALUE(bIsInitialized, bInIsInitialized);
}

void UMetaSoundOutputViewModel::SetOutputName(const FName& NewName) const
{
	const FName OldName = OutputName;
	if (const UMetaSoundViewModel* MetaSoundViewModel = Cast<UMetaSoundViewModel>(GetOuter()))
	{
		MetaSoundViewModel->SetOutputName(OldName, NewName);
	}
}

void UMetaSoundOutputViewModel::SetDataType(const FName& InDataType)
{
	if (!Metasound::Frontend::IDataTypeRegistry::Get().IsRegistered(InDataType))
	{
		UE_MVVM_BROADCAST_FIELD_VALUE_CHANGED(DataType);
		UE_LOGFMT(LogTechAudioTools, Warning, "Failed to set data type for {OutputName}. '{InDataType}' is not a registered data type.", OutputName, InDataType);
		return;
	}

	if (const UMetaSoundViewModel* MetaSoundViewModel = Cast<UMetaSoundViewModel>(GetOuter()))
	{
		MetaSoundViewModel->SetOutputDataType(OutputName, InDataType);
	}
}

void UMetaSoundOutputViewModel::SetIsArray(const bool bInIsArray)
{
	if (UMetaSoundViewModelConversionFunctions::IsArrayType(DataType))
	{
		SetDataType(TechAudioTools::MetaSound::GetAdjustedDataType(DataType, bInIsArray));
	}
}

void UMetaSoundOutputViewModel::SetIsConstructorPin(const bool bInIsConstructorPin)
{
	if (!UMetaSoundViewModelConversionFunctions::IsConstructorType(DataType))
	{
		UE_MVVM_BROADCAST_FIELD_VALUE_CHANGED(bIsConstructorPin);
		UE_LOGFMT(LogTechAudioTools, Warning, "Failed to set bIsConstructorPin on {OutputName}. '{DataType}' cannot be a constructor pin.", OutputName, DataType);
		return;
	}

	if (const UMetaSoundViewModel* MetaSoundViewModel = Cast<UMetaSoundViewModel>(GetOuter()))
	{
		MetaSoundViewModel->SetOutputIsConstructorPin(OutputName, bInIsConstructorPin);
	}
}

void UMetaSoundOutputViewModel::OnOutputNameChanged(const FName& NewName)
{
	UE_MVVM_SET_PROPERTY_VALUE(OutputName, NewName);
}

void UMetaSoundOutputViewModel::OnOutputDataTypeChanged(const FName& NewDataType)
{
	if (UE_MVVM_SET_PROPERTY_VALUE(DataType, NewDataType))
	{
		UE_MVVM_BROADCAST_FIELD_VALUE_CHANGED(bIsArray);
	}
}

void UMetaSoundOutputViewModel::OnOutputIsConstructorPinChanged(const bool bNewIsConstructorPin)
{
	UE_MVVM_SET_PROPERTY_VALUE(bIsConstructorPin, bNewIsConstructorPin);
}
