// Copyright Epic Games, Inc. All Rights Reserved.

#include "ViewModels/MetaSoundViewModelConversionFunctions.h"

#include "MetasoundFrontendSearchEngine.h"
#include "TechAudioToolsLog.h"
#include "Logging/StructuredLog.h"
#include "ViewModels/MetaSoundViewModel.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MetaSoundViewModelConversionFunctions)

UMetaSoundInputViewModel* UMetaSoundViewModelConversionFunctions::FindInputViewModelByName(const TArray<UMetaSoundInputViewModel*>& MetaSoundInputViewModels, const FName InputName)
{
	for (UMetaSoundInputViewModel* InputViewModel : MetaSoundInputViewModels)
	{
		if (InputViewModel->GetInputName() == InputName)
		{
			return InputViewModel;
		}
	}

	return nullptr;
}

UMetaSoundOutputViewModel* UMetaSoundViewModelConversionFunctions::FindOutputViewModelByName(const TArray<UMetaSoundOutputViewModel*>& MetaSoundOutputViewModels, const FName OutputName)
{
	for (UMetaSoundOutputViewModel* OutputViewModel : MetaSoundOutputViewModels)
	{
		if (OutputViewModel->GetOutputName() == OutputName)
		{
			return OutputViewModel;
		}
	}

	return nullptr;
}

FText UMetaSoundViewModelConversionFunctions::GetLiteralValueAsText(const FMetasoundFrontendLiteral& Literal)
{
	return FText::FromString(Literal.ToString());
}

bool UMetaSoundViewModelConversionFunctions::IsInterfaceMember(const FName& MemberName, const bool bInvert)
{
	FName InterfaceNamespace;
	FName ParamName;
	Audio::FParameterPath::SplitName(MemberName, InterfaceNamespace, ParamName);

	FMetasoundFrontendVersion FoundInterface;
	const bool bIsValidNamespace = Metasound::Frontend::ISearchEngine::Get().FindHighestInterfaceVersion(InterfaceNamespace, FoundInterface);
	const bool bResult = !InterfaceNamespace.IsNone() && bIsValidNamespace;

	return bInvert ? !bResult : bResult;
}

bool UMetaSoundViewModelConversionFunctions::IsArrayType(const FName& DataType, const bool bInvert)
{
	const FName AdjustedDataType = TechAudioTools::MetaSound::GetAdjustedDataType(DataType, true);
	const bool bIsValidDataType = Metasound::Frontend::IDataTypeRegistry::Get().IsRegistered(AdjustedDataType);
	return bInvert ? !bIsValidDataType : bIsValidDataType;
}

bool UMetaSoundViewModelConversionFunctions::IsConstructorType(const FName& DataType, const bool bInvert)
{
	using namespace Metasound::Frontend;

	FDataTypeRegistryInfo Info;
	if (!IDataTypeRegistry::Get().GetDataTypeInfo(DataType, Info))
	{
		UE_LOGFMT(LogTechAudioTools, Warning, "'{DataType}' is not a registered data type.", DataType);
		return false;
	}

	return bInvert ? !Info.bIsConstructorType : Info.bIsConstructorType;
}
