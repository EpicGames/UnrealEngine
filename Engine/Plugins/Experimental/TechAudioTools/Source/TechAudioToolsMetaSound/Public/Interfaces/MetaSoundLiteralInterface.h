// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Interface.h"

#include "MetaSoundLiteralInterface.generated.h"

class UMetaSoundInputViewModel;
class UMetaSoundViewModel;

#define UE_API TECHAUDIOTOOLSMETASOUND_API

/**
 * Used for retrieving specific viewmodels from a MetaSoundViewModel, such as inputs or outputs. Simplifies the process of initializing MetaSound
 * Literal Widgets by allowing the owning widget to call functions on entire containers of widgets that implement this interface, rather than
 * painstakingly initializing each widget one-by-one.
 *
 * Widgets implementing this interface are expected to provide an array of input names in order to request the required InputViewModels from the owning
 * widget's MetaSoundViewModel. Then, the widget can implement SetInputViewModels to correctly assign each InputViewModel received.
 *
 * To assign viewmodels, it is recommended to use either UMVVMView::SetViewModelByClass (if there aren't multiple viewmodels of the same class) or
 * UMVVMView::SetViewModel (if there are multiple viewmodels of the same class). If using UMVVMView::SetViewModel, you must ensure the name of the
 * viewmodel passed into the function matches the name of the corresponding viewmodel instance *exactly* or else the assignment will fail. It is
 * recommended to expose a name variable to designers rather than hardcode a name value.
 *
 * @see UMetaSoundLiteralWidget_Float for an example implementation.
 */
UINTERFACE(MinimalAPI, Blueprintable, DisplayName = "MetaSound Literal Widget Interface")
class UMetaSoundLiteralWidgetInterface : public UInterface
{
	GENERATED_BODY()
};

class IMetaSoundLiteralWidgetInterface
{
	GENERATED_BODY()

public:
	// Returns an array containing the names of the MetaSound Input Viewmodels required for this widget.
	UFUNCTION(BlueprintCallable, BlueprintNativeEvent, DisplayName = "Get Input Viewmodel Names", Category = "MetaSound Literal")
	UE_API TArray<FName> GetInputViewModelNames() const;
	virtual TArray<FName> GetInputViewModelNames_Implementation() const { return TArray<FName>(); }

	// Sets the MetaSound input viewmodels this widget binds to.
	UFUNCTION(BlueprintCallable, BlueprintNativeEvent, DisplayName = "Set Input Viewmodels", Category = "MetaSound Literal")
	UE_API void SetInputViewModels(UPARAM(DisplayName = "Input Viewmodels") const TMap<FName, UMetaSoundInputViewModel*>& InputViewModels);
	virtual void SetInputViewModels_Implementation(const TMap<FName, UMetaSoundInputViewModel*>& InputViewModels) {}
};

#undef UE_API
