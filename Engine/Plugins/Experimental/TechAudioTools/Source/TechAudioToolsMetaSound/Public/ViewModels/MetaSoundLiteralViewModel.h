// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MetasoundFrontendLiteral.h"
#include "MVVMViewModelBase.h"
#include "TechAudioToolsTypes.h"

#include "MetaSoundLiteralViewModel.generated.h"

class UMetaSoundBuilderBase;
class UTechAudioToolsFloatMapping;

#define UE_API TECHAUDIOTOOLSMETASOUND_API

/**
 * Viewmodel for binding a boolean-type MetaSound Literal with a widget.
 */
UCLASS(BlueprintType, EditInlineNew, DefaultToInstanced, CollapseCategories, DisplayName = "MetaSound Literal Boolean Viewmodel")
class UMetaSoundLiteralViewModel_Boolean : public UMVVMViewModelBase
{
	GENERATED_BODY()

protected:
	// The bound MetaSound Literal value. Should be bound using a two-way binding with the Literal variable of a MetaSoundInputViewModel.
	UPROPERTY(BlueprintReadWrite, Transient, FieldNotify, Setter, Category = "Boolean Literal", meta = (AllowPrivateAccess))
	FMetasoundFrontendLiteral Literal;

	// The value of the bound Literal as a bool.
	UPROPERTY(BlueprintReadWrite, Transient, FieldNotify, Getter, Setter, Category = "Boolean Literal", meta = (AllowPrivateAccess))
	bool SourceValue = false;

public:
	const FMetasoundFrontendLiteral& GetLiteral() const { return Literal; }
	bool GetSourceValue() const { return SourceValue; }

	UE_API void SetLiteral(const FMetasoundFrontendLiteral& InLiteral);
	UE_API void SetSourceValue(bool InSourceValue);

	virtual void PostInitProperties() override;

#if WITH_EDITOR
	virtual void PostEditUndo() override;
#endif
};

/**
 * Viewmodel for binding a boolean array-type MetaSound Literal with a widget.
 */
UCLASS(BlueprintType, EditInlineNew, DefaultToInstanced, CollapseCategories, DisplayName = "MetaSound Literal Boolean Array Viewmodel")
class UMetaSoundLiteralViewModel_BooleanArray : public UMVVMViewModelBase
{
	GENERATED_BODY()

protected:
	// The bound MetaSound Literal value. Should be bound using a two-way binding with the Literal variable of a MetaSoundInputViewModel.
	UPROPERTY(BlueprintReadWrite, Transient, FieldNotify, Setter, Category = "Boolean Array Literal", meta = (AllowPrivateAccess))
	FMetasoundFrontendLiteral Literal;

	// The value of the bound Literal as a boolean array.
	UPROPERTY(BlueprintReadWrite, Transient, FieldNotify, Getter, Setter, Category = "Boolean Array Literal", meta = (AllowPrivateAccess))
	TArray<bool> SourceValue;

public:
	const FMetasoundFrontendLiteral& GetLiteral() const { return Literal; }
	TArray<bool> GetSourceValue() const { return SourceValue; }

	UE_API void SetLiteral(const FMetasoundFrontendLiteral& InLiteral);
	UE_API void SetSourceValue(const TArray<bool>& InSourceValue);

	virtual void PostInitProperties() override;

#if WITH_EDITOR
	virtual void PostEditUndo() override;
#endif
};

/**
 * Viewmodel for binding an integer-type MetaSound Literal with a widget.
 */
UCLASS(BlueprintType, EditInlineNew, DefaultToInstanced, CollapseCategories, DisplayName = "MetaSound Literal Integer Viewmodel")
class UMetaSoundLiteralViewModel_Integer : public UMVVMViewModelBase
{
	GENERATED_BODY()

protected:
	// The bound MetaSound Literal value. Should be bound using a two-way binding with the Literal variable of a MetaSoundInputViewModel.
	UPROPERTY(BlueprintReadWrite, Transient, FieldNotify, Setter, Category = "Integer Literal", meta = (AllowPrivateAccess))
	FMetasoundFrontendLiteral Literal;

	// The value of the bound Literal as an integer.
	UPROPERTY(BlueprintReadWrite, Transient, FieldNotify, Getter, Setter, Category = "Integer Literal", meta = (AllowPrivateAccess))
	int32 SourceValue = 0;

	// The SourceValue normalized to a 0-1 range based on the set Range values. Used for UI like sliders and knobs that use a normalized range.
	UPROPERTY(BlueprintReadWrite, Transient, FieldNotify, Getter, Setter, Category = "Integer Literal", meta = (AllowPrivateAccess))
	float NormalizedValue = 0.f;

	// Sets the min and max values of the UI range. Range bounds should be closed (either Inclusive or Exclusive) and not open.
	UPROPERTY(BlueprintReadWrite, EditAnywhere, FieldNotify, Category = "Integer Literal", meta = (AllowPrivateAccess))
	FInt32Range Range;

public:
	UFUNCTION(BlueprintCallable, FieldNotify, Category = "Integer Literal")
	UE_API float GetStepSize() const;

	const FMetasoundFrontendLiteral& GetLiteral() const { return Literal; }
	int32 GetSourceValue() const { return SourceValue; }
	float GetNormalizedValue() const { return NormalizedValue; }

	UE_API void SetLiteral(const FMetasoundFrontendLiteral& InLiteral);
	UE_API void SetSourceValue(int32 InSourceValue);
	UE_API void SetNormalizedValue(float InNormalizedValue);

	virtual void PostInitProperties() override;

#if WITH_EDITOR
	virtual void PostEditUndo() override;
#endif

private:
	bool bIsUpdating = false;
};

/**
 * Viewmodel for binding an integer array-type MetaSound Literal with a widget.
 */
UCLASS(BlueprintType, EditInlineNew, DefaultToInstanced, CollapseCategories, DisplayName = "MetaSound Literal Integer Array Viewmodel")
class UMetaSoundLiteralViewModel_IntegerArray : public UMVVMViewModelBase
{
	GENERATED_BODY()

protected:
	// The bound MetaSound Literal value. Should be bound using a two-way binding with the Literal variable of a MetaSoundInputViewModel.
	UPROPERTY(BlueprintReadWrite, Transient, FieldNotify, Setter, Category = "Integer Array Literal", meta = (AllowPrivateAccess))
	FMetasoundFrontendLiteral Literal;

	// The value of the bound Literal as an integer array.
	UPROPERTY(BlueprintReadWrite, Transient, FieldNotify, Getter, Setter, Category = "Integer Array Literal", meta = (AllowPrivateAccess))
	TArray<int32> SourceValue;

public:
	const FMetasoundFrontendLiteral& GetLiteral() const { return Literal; }
	TArray<int32> GetSourceValue() const { return SourceValue; }

	UE_API void SetLiteral(const FMetasoundFrontendLiteral& InLiteral);
	UE_API void SetSourceValue(const TArray<int32>& InSourceValue);

	virtual void PostInitProperties() override;

#if WITH_EDITOR
	virtual void PostEditUndo() override;
#endif
};

/**
 * Viewmodel for converting a float-type MetaSound Literal between Source, Normalized, and Display values.
 *
 * Source values refer to the units or range expected by the MetaSound Literal. Display values refer to a configurable conversion of the Source value.
 * For instance, a "Volume" input might expect a linear gain value in the MetaSound, but users may wish to display the value as decibels.
 *
 * The Normalized value is to be used for UI elements such as sliders and knobs.
 *
 * The Literal variable belonging to this viewmodel is expected to be bound to the Literal variable of a MetaSoundInputViewModel using a two-way
 * binding for bidirectional updates. Doing so will ensure all updates to the input in the MetaSound Editor are reflected in your widget and vice versa.
 *
 * This viewmodel should most likely use the Create Instance setting, along with the Expose Instance in Editor option. This allows designers to easily
 * configure units and range settings in the Details panel of instances of the owning widget.
 *
 * @see UTechAudioToolsFloatMapping for unit conversion and range mapping logic.
 */
UCLASS(BlueprintType, EditInlineNew, DefaultToInstanced, CollapseCategories, DisplayName = "MetaSound Literal Float Viewmodel")
class UMetaSoundLiteralViewModel_Float : public UMVVMViewModelBase
{
	GENERATED_BODY()

protected:
	// If true, the NormalizedValue will be mapped to the DisplayRange, else mapped to the SourceRange.
	UPROPERTY(BlueprintReadWrite, EditAnywhere, FieldNotify, Getter = "IsShowingDisplayValues", Setter = "SetShowDisplayValues", Category = "Float Literal")
	bool bShowDisplayValues = true;

	// The bound MetaSound Literal value. Should be bound using a two-way binding with the Literal variable of a MetaSoundInputViewModel.
	UPROPERTY(BlueprintReadWrite, Transient, FieldNotify, Setter, Category = "Float Literal", meta = (AllowPrivateAccess))
	FMetasoundFrontendLiteral Literal;

	// The value of the bound Literal as a float.
	UPROPERTY(BlueprintReadWrite, Transient, FieldNotify, Getter, Setter, Category = "Float Literal", meta = (AllowPrivateAccess))
	float SourceValue = 0.f;

	/**
	 * The normalized (0-1) representation of the value currently surfaced to the UI. If bShowDisplayValues is true, this is DisplayValue mapped into
	 * 0-1 space. Otherwise, it's SourceValue mapped into 0-1 space. Used by knobs/sliders purely for visual feedback.
	 */
	UPROPERTY(BlueprintReadWrite, Transient, FieldNotify, Getter, Setter, Category = "Float Literal", meta = (AllowPrivateAccess))
	float NormalizedValue = 0.f;

	// The user-facing display value (e.g. dB, Hz, semitones). Can be modified and is automatically converted to the expected SourceRange.
	UPROPERTY(BlueprintReadWrite, Transient, FieldNotify, Getter, Setter, Category = "Float Literal", meta = (AllowPrivateAccess))
	float DisplayValue = 0.f;

	// Mapping pair used to convert between float literal, normalized, and display values.
	UPROPERTY(EditAnywhere, Instanced, Category = "Float Range Settings", meta = (AllowPrivateAccess))
	TObjectPtr<UTechAudioToolsFloatMapping> RangeValues;

public:
	// Returns the range minimum of the SourceRange.
	UFUNCTION(BlueprintCallable, BlueprintPure, FieldNotify, Category = "TechAudioTools")
	UE_API float GetSourceRangeMin() const;

	// Returns the range maximum of the SourceRange.
	UFUNCTION(BlueprintCallable, BlueprintPure, FieldNotify, Category = "TechAudioTools")
	UE_API float GetSourceRangeMax() const;

	// Returns the units associated with the SourceValue.
	UFUNCTION(BlueprintCallable, BlueprintPure, FieldNotify, Category = "TechAudioTools")
	UE_API ETechAudioToolsFloatUnit GetSourceUnits() const;

	// Returns the range minimum of the DisplayRange.
	UFUNCTION(BlueprintCallable, BlueprintPure, FieldNotify, Category = "TechAudioTools")
	UE_API float GetDisplayRangeMin() const;

	// Returns the range maximum of the DisplayRange.
	UFUNCTION(BlueprintCallable, BlueprintPure, FieldNotify, Category = "TechAudioTools")
	UE_API float GetDisplayRangeMax() const;

	// Returns the units associated with the DisplayValue.
	UFUNCTION(BlueprintCallable, BlueprintPure, FieldNotify, Category = "TechAudioTools")
	UE_API ETechAudioToolsFloatUnit GetDisplayUnits() const;

	UE_API explicit UMetaSoundLiteralViewModel_Float(const FObjectInitializer& ObjectInitializer);

	bool IsShowingDisplayValues() const { return bShowDisplayValues; }
	const FMetasoundFrontendLiteral& GetLiteral() const { return Literal; }
	float GetSourceValue() const { return SourceValue; }
	float GetNormalizedValue() const { return NormalizedValue; }
	float GetDisplayValue() const { return DisplayValue; }

	UE_API void SetShowDisplayValues(bool bInIsShowingDisplayValues);
	UE_API void SetLiteral(const FMetasoundFrontendLiteral& InLiteral);
	UE_API void SetSourceValue(float InSourceValue);
	UE_API void SetNormalizedValue(float InNormalizedValue);
	UE_API void SetDisplayValue(float InDisplayValue);

	virtual void PostInitProperties() override;

#if WITH_EDITOR
	virtual void PostEditUndo() override;
#endif

private:
	bool bIsUpdating = false;
};

/**
 * Viewmodel for binding a float array-type MetaSound Literal with a widget.
 */
UCLASS(BlueprintType, EditInlineNew, DefaultToInstanced, CollapseCategories, DisplayName = "MetaSound Literal Float Array Viewmodel")
class UMetaSoundLiteralViewModel_FloatArray : public UMVVMViewModelBase
{
	GENERATED_BODY()

protected:
	// The bound MetaSound Literal value. Should be bound using a two-way binding with the Literal variable of a MetaSoundInputViewModel.
	UPROPERTY(BlueprintReadWrite, Transient, FieldNotify, Setter, Category = "Float Array Literal", meta = (AllowPrivateAccess))
	FMetasoundFrontendLiteral Literal;

	// The value of the bound Literal as a float array.
	UPROPERTY(BlueprintReadWrite, Transient, FieldNotify, Getter, Setter, Category = "Float Array Literal", meta = (AllowPrivateAccess))
	TArray<float> SourceValue;

public:
	const FMetasoundFrontendLiteral& GetLiteral() const { return Literal; }
	TArray<float> GetSourceValue() const { return SourceValue; }

	UE_API void SetLiteral(const FMetasoundFrontendLiteral& InLiteral);
	UE_API void SetSourceValue(const TArray<float>& InSourceValue);

	virtual void PostInitProperties() override;

#if WITH_EDITOR
	virtual void PostEditUndo() override;
#endif
};

/**
 * Viewmodel for binding a string-type MetaSound Literal with a widget.
 */
UCLASS(BlueprintType, EditInlineNew, DefaultToInstanced, CollapseCategories, DisplayName = "MetaSound Literal String Viewmodel")
class UMetaSoundLiteralViewModel_String : public UMVVMViewModelBase
{
	GENERATED_BODY()

protected:
	// The bound MetaSound Literal value. Should be bound using a two-way binding with the Literal variable of a MetaSoundInputViewModel.
	UPROPERTY(BlueprintReadWrite, Transient, FieldNotify, Setter, Category = "String Literal", meta = (AllowPrivateAccess))
	FMetasoundFrontendLiteral Literal;

	// The value of the bound Literal as a string.
	UPROPERTY(BlueprintReadWrite, Transient, FieldNotify, Getter, Setter, Category = "String Literal", meta = (AllowPrivateAccess))
	FString SourceValue;

public:
	const FMetasoundFrontendLiteral& GetLiteral() const { return Literal; }
	FString GetSourceValue() const { return SourceValue; }

	UE_API void SetLiteral(const FMetasoundFrontendLiteral& InLiteral);
	UE_API void SetSourceValue(const FString& InSourceValue);

	virtual void PostInitProperties() override;

#if WITH_EDITOR
	virtual void PostEditUndo() override;
#endif
};

/**
 * Viewmodel for binding a string array-type MetaSound Literal with a widget.
 */
UCLASS(BlueprintType, EditInlineNew, DefaultToInstanced, CollapseCategories, DisplayName = "MetaSound Literal String Array Viewmodel")
class UMetaSoundLiteralViewModel_StringArray : public UMVVMViewModelBase
{
	GENERATED_BODY()

protected:
	// The bound MetaSound Literal value. Should be bound using a two-way binding with the Literal variable of a MetaSoundInputViewModel.
	UPROPERTY(BlueprintReadWrite, Transient, FieldNotify, Setter, Category = "String Array Literal", meta = (AllowPrivateAccess))
	FMetasoundFrontendLiteral Literal;

	// The value of the bound Literal as a string array.
	UPROPERTY(BlueprintReadWrite, Transient, FieldNotify, Getter, Setter, Category = "String Array Literal", meta = (AllowPrivateAccess))
	TArray<FString> SourceValue;

public:
	const FMetasoundFrontendLiteral& GetLiteral() const { return Literal; }
	TArray<FString> GetSourceValue() const { return SourceValue; }

	UE_API void SetLiteral(const FMetasoundFrontendLiteral& InLiteral);
	UE_API void SetSourceValue(const TArray<FString>& InSourceValue);

	virtual void PostInitProperties() override;

#if WITH_EDITOR
	virtual void PostEditUndo() override;
#endif
};

/**
 * Viewmodel for binding an object-type MetaSound Literal with a widget.
 */
UCLASS(BlueprintType, EditInlineNew, DefaultToInstanced, CollapseCategories, DisplayName = "MetaSound Literal Object Viewmodel")
class UMetaSoundLiteralViewModel_Object : public UMVVMViewModelBase
{
	GENERATED_BODY()

protected:
	// The bound MetaSound Literal value. Should be bound using a two-way binding with the Literal variable of a MetaSoundInputViewModel.
	UPROPERTY(BlueprintReadWrite, Transient, FieldNotify, Setter, Category = "Object Literal", meta = (AllowPrivateAccess))
	FMetasoundFrontendLiteral Literal;

	// The value of the bound Literal as an object.
	UPROPERTY(BlueprintReadWrite, Transient, FieldNotify, Getter, Setter, Category = "Object Literal", meta = (AllowPrivateAccess))
	TObjectPtr<UObject> SourceValue;

public:
	const FMetasoundFrontendLiteral& GetLiteral() const { return Literal; }
	UObject* GetSourceValue() const { return SourceValue.Get(); }

	UE_API void SetLiteral(const FMetasoundFrontendLiteral& InLiteral);
	UE_API void SetSourceValue(UObject* InSourceValue);

	virtual void PostInitProperties() override;

#if WITH_EDITOR
	virtual void PostEditUndo() override;
#endif
};

/**
 * Viewmodel for binding an object array-type MetaSound Literal with a widget.
 */
UCLASS(BlueprintType, EditInlineNew, DefaultToInstanced, CollapseCategories, DisplayName = "MetaSound Literal Object Array Viewmodel")
class UMetaSoundLiteralViewModel_ObjectArray : public UMVVMViewModelBase
{
	GENERATED_BODY()

protected:
	// The bound MetaSound Literal value. Should be bound using a two-way binding with the Literal variable of a MetaSoundInputViewModel.
	UPROPERTY(BlueprintReadWrite, Transient, FieldNotify, Setter, Category = "Object Array Literal", meta = (AllowPrivateAccess))
	FMetasoundFrontendLiteral Literal;

	// The value of the bound Literal as an object array.
	UPROPERTY(BlueprintReadWrite, Transient, FieldNotify, Getter, Setter, Category = "Object Array Literal", meta = (AllowPrivateAccess))
	TArray<TObjectPtr<UObject>> SourceValue;

public:
	const FMetasoundFrontendLiteral& GetLiteral() const { return Literal; }
	TArray<TObjectPtr<UObject>> GetSourceValue() const { return SourceValue; }

	UE_API void SetLiteral(const FMetasoundFrontendLiteral& InLiteral);
	UE_API void SetSourceValue(const TArray<TObjectPtr<UObject>>& InSourceValue);

	virtual void PostInitProperties() override;

#if WITH_EDITOR
	virtual void PostEditUndo() override;
#endif
};

#undef UE_API
