// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MetasoundFrontendDocumentBuilder.h"
#include "MVVMViewModelBase.h"
#include "Templates/SubclassOf.h"

#include "MetaSoundViewModel.generated.h"

#define UE_API TECHAUDIOTOOLSMETASOUND_API

class UMetaSoundBuilderBase;
class UMetaSoundInputViewModel;
class UMetaSoundOutputViewModel;

/**
 * The base class for MetaSound viewmodels. Used for binding metadata and member inputs/outputs of a MetaSound to widgets in UMG.
 * Can be initialized using a MetaSound Builder or a MetaSound asset. Creates member viewmodels for each input and output in the
 * MetaSound upon initialization.
 */
UCLASS(MinimalAPI, DisplayName = "MetaSound Viewmodel")
class UMetaSoundViewModel : public UMVVMViewModelBase
{
	GENERATED_BODY()

protected:
	// True if this MetaSound Viewmodel has been initialized.
	UPROPERTY(BlueprintReadOnly, FieldNotify, Getter = "IsInitialized", Category = "MetaSound Viewmodel", meta = (AllowPrivateAccess))
	bool bIsInitialized = false;

	// True if the initialized MetaSound is a preset.
	UPROPERTY(BlueprintReadOnly, FieldNotify, Getter = "IsPreset", Category = "MetaSound Viewmodel", meta = (AllowPrivateAccess))
	bool bIsPreset = false;

public:
	// Returns the object name of the initialized builder.
	UFUNCTION(BlueprintCallable, FieldNotify, Category = "MetaSound Viewmodel")
	FString GetBuilderName() const { return Builder.GetName(); }

	// Contains MetaSound Input Viewmodels for each input of the initialized MetaSound.
	UFUNCTION(BlueprintCallable, FieldNotify, DisplayName = "Get Input Viewmodels", Category = "MetaSound Viewmodel")
	UE_API virtual TArray<UMetaSoundInputViewModel*> GetInputViewModels() const;

	// Returns the input viewmodel with the given name if it exists, else returns nullptr.
	UFUNCTION(BlueprintCallable, DisplayName = "Find Input Viewmodel", Category = "MetaSound Viewmodel")
	UE_API UMetaSoundInputViewModel* FindInputViewModel(const FName InputViewModelName) const;

	// Contains MetaSound Output Viewmodels for each output of the initialized MetaSound.
	UFUNCTION(BlueprintCallable, FieldNotify, DisplayName = "Get Output Viewmodels", Category = "MetaSound Viewmodel")
	UE_API virtual TArray<UMetaSoundOutputViewModel*> GetOutputViewModels() const;

	// Returns the output viewmodel with the given name if it exists, else returns nullptr.
	UFUNCTION(BlueprintCallable, DisplayName = "Find Output Viewmodel", Category = "MetaSound Viewmodel")
	UE_API UMetaSoundOutputViewModel* FindOutputViewModel(const FName OutputViewModelName) const;

	// Initializes the viewmodel using the given MetaSound asset.
	UFUNCTION(BlueprintCallable, DisplayName = "Initialize MetaSound", Category = "Audio|MetaSound Viewmodel")
	UE_API virtual void InitializeMetaSound(const TScriptInterface<IMetaSoundDocumentInterface> InMetaSound);

	// Initializes the viewmodel using the given builder.
	UFUNCTION(BlueprintCallable, DisplayName = "Initialize Builder", Category = "Audio|MetaSound Viewmodel")
	UE_API virtual void Initialize(UMetaSoundBuilderBase* InBuilder);

	// Resets this MetaSoundViewModel instance to an uninitialized state.
	UFUNCTION(BlueprintCallable, Category = "Audio|MetaSound Viewmodel")
	UE_API virtual void Reset();

	bool IsInitialized() const { return bIsInitialized; }
	bool IsPreset() const { return bIsPreset; }

	UE_API void SetIsInitialized(const bool bInIsInitialized);

	UE_API void SetInputName(const FName& OldName, const FName& NewName) const;
	UE_API void SetInputDataType(const FName& InputName, const FName& DataType) const;
	UE_API void SetInputDefaultLiteral(const FName& InputName, const FMetasoundFrontendLiteral& DefaultLiteral) const;
	UE_API void SetInputOverridesDefault(const FName& InputName, bool bOverridesDefault) const;
	UE_API void SetInputIsConstructorPin(const FName& InputName, bool bIsConstructorPin) const;

	UE_API void SetOutputName(const FName& OldName, const FName& NewName) const;
	UE_API void SetOutputDataType(const FName& OutputName, const FName& DataType) const;
	UE_API void SetOutputIsConstructorPin(const FName& OutputName, bool bIsConstructorPin) const;

	UFUNCTION()
	UE_API void OnInputAdded(FName VertexName, FName DataType);
	UFUNCTION()
	UE_API void OnInputRemoved(FName VertexName, FName DataType);
	UFUNCTION()
	UE_API void OnInputNameChanged(FName OldName, FName NewName);
	UFUNCTION()
	UE_API void OnInputDataTypeChanged(FName VertexName, FName DataType);
	UFUNCTION()
	UE_API void OnInputDefaultChanged(FName VertexName, FMetasoundFrontendLiteral LiteralValue, FName PageName, EMetaSoundLiteralChangeType ChangeType = EMetaSoundLiteralChangeType::Invalid);
	UFUNCTION()
	UE_API void OnInputInheritsDefaultChanged(FName VertexName, bool bInheritsDefault);
	UFUNCTION()
	UE_API void OnInputIsConstructorPinChanged(FName VertexName, bool bIsConstructorPin);

	UFUNCTION()
	UE_API void OnOutputAdded(FName VertexName, FName DataType);
	UFUNCTION()
	UE_API void OnOutputRemoved(FName VertexName, FName DataType);
	UFUNCTION()
	UE_API void OnOutputNameChanged(FName OldName, FName NewName);
	UFUNCTION()
	UE_API void OnOutputDataTypeChanged(FName VertexName, FName DataType);
	UFUNCTION()
	UE_API void OnOutputIsConstructorPinChanged(FName VertexName, bool bIsConstructorPin);

protected:
	UE_API void CreateMemberViewModels();
	UE_API virtual UMetaSoundInputViewModel* CreateInputViewModel(const FMetasoundFrontendClassInput& InInput);
	UE_API virtual UMetaSoundOutputViewModel* CreateOutputViewModel(const FMetasoundFrontendClassOutput& InOutput);

	virtual TSubclassOf<UMetaSoundInputViewModel> GetInputViewModelClass() const;
	virtual TSubclassOf<UMetaSoundOutputViewModel> GetOutputViewModelClass() const;

	UPROPERTY(Transient)
	TObjectPtr<UMetaSoundBuilderBase> Builder;

	UPROPERTY(Transient)
	TMap<FName, TObjectPtr<UMetaSoundInputViewModel>> InputViewModels;

	UPROPERTY(Transient)
	TMap<FName, TObjectPtr<UMetaSoundOutputViewModel>> OutputViewModels;
};

/**
 * Viewmodel class for MetaSound inputs. Allows widgets in UMG to bind to MetaSound literals. Useful for creating knobs, sliders, and other
 * widgets for setting MetaSound input parameters.
 */
UCLASS(MinimalAPI, DisplayName = "MetaSound Input Viewmodel")
class UMetaSoundInputViewModel : public UMVVMViewModelBase
{
	GENERATED_BODY()

protected:
	// True if this MetaSoundInputViewModel has been initialized.
	UPROPERTY(BlueprintReadOnly, FieldNotify, Getter = "IsInitialized", Category = "MetaSound Input", meta = (AllowPrivateAccess))
	bool bIsInitialized = false;

	// The name of the initialized MetaSound input.
	UPROPERTY(BlueprintReadWrite, EditAnywhere, FieldNotify, Getter, Setter, Category = "MetaSound Input", meta = (AllowPrivateAccess))
	FName InputName;

	// The data type of the initialized MetaSound input.
	UPROPERTY(BlueprintReadWrite, FieldNotify, Getter, Setter, Category = "MetaSound Input", meta = (AllowPrivateAccess))
	FName DataType;

	// True if the initialized MetaSound input is an array.
	UPROPERTY(BlueprintReadWrite, FieldNotify, Getter = "IsArray", Setter = "SetIsArray", Category = "MetaSound Input", meta = (AllowPrivateAccess))
	bool bIsArray;

	// True if the initialized MetaSound input is a constructor pin.
	UPROPERTY(BlueprintReadWrite, FieldNotify, Getter = "IsConstructorPin", Setter = "SetIsConstructorPin", Category = "MetaSound Input", meta = (AllowPrivateAccess))
	bool bIsConstructorPin;

	// The MetaSound Literal belonging to the initialized MetaSound input.
	UPROPERTY(BlueprintReadWrite, FieldNotify, Getter, Setter, Category = "MetaSound Input", meta = (AllowPrivateAccess))
	FMetasoundFrontendLiteral Literal;

	// Returns the Literal Type belonging to the initialized MetaSound input.
	UPROPERTY(BlueprintReadOnly, FieldNotify, Getter, Category = "MetaSound Input", meta = (AllowPrivateAccess))
	EMetasoundFrontendLiteralType LiteralType;

	// True if the initialized MetaSound input overrides its value from a parent graph.
	UPROPERTY(BlueprintReadWrite, FieldNotify, Getter = "OverridesDefault", Setter = "SetOverridesDefault", Category = "MetaSound Input", meta = (AllowPrivateAccess))
	bool bOverridesDefault;

public:
	bool IsInitialized() const { return bIsInitialized; }
	FName GetInputName() const { return InputName; }
	FName GetDataType() const { return DataType; }
	bool IsArray() const { return DataType.ToString().EndsWith(METASOUND_DATA_TYPE_NAME_ARRAY_TYPE_SPECIFIER); }
	bool IsConstructorPin() const { return bIsConstructorPin; }
	FMetasoundFrontendLiteral GetLiteral() const { return Literal; }
	EMetasoundFrontendLiteralType GetLiteralType() const { return Literal.GetType(); }
	bool OverridesDefault() const { return bOverridesDefault; }

	UE_API void SetIsInitialized(const bool bInIsInitialized);
	UE_API void SetInputName(const FName& NewName);
	UE_API void SetDataType(const FName& InDataType);
	UE_API void SetIsArray(bool bInIsArray);
	UE_API void SetIsConstructorPin(bool bInIsConstructorPin);
	UE_API void SetLiteral(const FMetasoundFrontendLiteral& InLiteral);
	UE_API void SetOverridesDefault(bool bInOverridesDefaultValue);

	UE_API void OnInputNameChanged(const FName& NewName);
	UE_API void OnInputDataTypeChanged(const FName& NewDataType);
	UE_API void OnInputDefaultChanged(const FMetasoundFrontendLiteral& LiteralValue, const FName& PageName, EMetaSoundLiteralChangeType ChangeType = EMetaSoundLiteralChangeType::Invalid);
	UE_API void OnInputInheritsDefaultChanged(const bool bInheritsDefault);
	UE_API void OnInputIsConstructorPinChanged(const bool bNewIsConstructorPin);
};

/**
 * Viewmodel class for MetaSound outputs. Allows widgets in UMG to bind to data from a MetaSound output. Useful for driving visual parameters
 * using MetaSound outputs.
 */
UCLASS(MinimalAPI, DisplayName = "MetaSound Output Viewmodel")
class UMetaSoundOutputViewModel : public UMVVMViewModelBase
{
	GENERATED_BODY()

protected:
	// True if this MetaSoundOutputViewModel has been initialized.
	UPROPERTY(BlueprintReadOnly, FieldNotify, Category = "MetaSound Output", meta = (AllowPrivateAccess))
	bool bIsInitialized = false;

	// The name of the initialized MetaSound output.
	UPROPERTY(BlueprintReadWrite, EditAnywhere, FieldNotify, Setter, Category = "MetaSound Output", meta = (AllowPrivateAccess))
	FName OutputName;

	// The data type of the initialized MetaSound output.
	UPROPERTY(BlueprintReadWrite, FieldNotify, Setter, Category = "MetaSound Output", meta = (AllowPrivateAccess))
	FName DataType;

	// True if the initialized MetaSound output is an array.
	UPROPERTY(BlueprintReadWrite, FieldNotify, Getter = "IsArray", Setter = "SetIsArray", Category = "MetaSound Output", meta = (AllowPrivateAccess))
	bool bIsArray;

	// True if the initialized MetaSound output is a constructor pin.
	UPROPERTY(BlueprintReadWrite, FieldNotify, Getter = "IsConstructorPin", Setter = "SetIsConstructorPin", Category = "MetaSound Output", meta = (AllowPrivateAccess))
	bool bIsConstructorPin;

public:
	bool IsInitialized() const { return bIsInitialized; }
	FName GetOutputName() const { return OutputName; }
	FName GetDataType() const { return DataType; }
	bool IsArray() const { return DataType.ToString().EndsWith(METASOUND_DATA_TYPE_NAME_ARRAY_TYPE_SPECIFIER); }
	bool IsConstructorPin() const { return bIsConstructorPin; }

	UE_API void SetIsInitialized(const bool bInIsInitialized);
	UE_API void SetOutputName(const FName& NewName) const;
	UE_API void SetDataType(const FName& InDataType);
	UE_API void SetIsArray(bool bInIsArray);
	UE_API void SetIsConstructorPin(bool bInIsConstructorPin);

	UE_API void OnOutputNameChanged(const FName& NewName);
	UE_API void OnOutputDataTypeChanged(const FName& NewDataType);
	UE_API void OnOutputIsConstructorPinChanged(const bool bNewIsConstructorPin);
};

#undef UE_API
