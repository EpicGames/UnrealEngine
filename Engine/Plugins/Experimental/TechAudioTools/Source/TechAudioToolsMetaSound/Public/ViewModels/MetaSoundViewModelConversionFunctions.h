// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Kismet/BlueprintFunctionLibrary.h"
#include "MetasoundDataReferenceMacro.h"

#include "MetaSoundViewModelConversionFunctions.generated.h"

struct FMetasoundFrontendLiteral;
class UMetaSoundInputViewModel;
class UMetaSoundOutputViewModel;

namespace TechAudioTools::MetaSound
{
	// Returns the given data type with the array type specifier added or removed to match bIsArray.
	inline FName GetAdjustedDataType(const FName& DataType, const bool bIsArray)
	{
		const FString DataTypeString = DataType.ToString();
		if (bIsArray)
		{
			// Append the array type specifier if it isn't already there
			if (!DataTypeString.EndsWith(METASOUND_DATA_TYPE_NAME_ARRAY_TYPE_SPECIFIER))
			{
				return FName(DataTypeString + METASOUND_DATA_TYPE_NAME_ARRAY_TYPE_SPECIFIER);
			}
			return DataType;
		}
		if (DataTypeString.EndsWith(METASOUND_DATA_TYPE_NAME_ARRAY_TYPE_SPECIFIER))
		{
			// Remove the array type specifier
			FString BaseDataType;
			DataTypeString.Split(TEXT(":"), &BaseDataType, nullptr);
			return FName(BaseDataType);
		}
		return DataType;
	}
} // TechAudioTools::MetaSound

/**
 * Collection of conversion functions to use with MetaSound Viewmodels.
 */
UCLASS(MinimalAPI)
class UMetaSoundViewModelConversionFunctions : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:
	// Searches the given array of MetaSoundInputViewModels for the specified input. Returns nullptr if unable to find a matching viewmodel.
	UFUNCTION(BlueprintCallable, BlueprintPure, DisplayName = "Find MetaSound Input Viewmodel by Name", Category = "TechAudioTools")
	static TECHAUDIOTOOLSMETASOUND_API UMetaSoundInputViewModel* FindInputViewModelByName(const TArray<UMetaSoundInputViewModel*>& MetaSoundInputViewModels, const FName InputName);

	// Searches the given array of MetaSoundOutputViewModels for the specified output. Returns nullptr if unable to find a matching viewmodel.
	UFUNCTION(BlueprintCallable, BlueprintPure, DisplayName = "Find MetaSound Output Viewmodel by Name", Category = "TechAudioTools")
	static TECHAUDIOTOOLSMETASOUND_API UMetaSoundOutputViewModel* FindOutputViewModelByName(const TArray<UMetaSoundOutputViewModel*>& MetaSoundOutputViewModels, const FName OutputName);

	// Returns the value of the given MetaSound Literal as a text value.
	UFUNCTION(BlueprintCallable, BlueprintPure, DisplayName = "Get MetaSound Literal Value as Text", Category = "TechAudioTools")
	static TECHAUDIOTOOLSMETASOUND_API FText GetLiteralValueAsText(const FMetasoundFrontendLiteral& Literal);

	// Checks whether the given member name belongs to a registered MetaSound interface.
	UFUNCTION(BlueprintCallable, BlueprintPure, DisplayName = "Is MetaSound Interface Member", Category = "TechAudioTools")
	static TECHAUDIOTOOLSMETASOUND_API bool IsInterfaceMember(const FName& MemberName, UPARAM(DisplayName = "NOT") bool bInvert = false);

	// Returns true if the given MetaSound data type is registered and can be used as an array type.
	UFUNCTION(BlueprintCallable, BlueprintPure, DisplayName = "Is MetaSound Array Type", Category = "TechAudioTools")
	static TECHAUDIOTOOLSMETASOUND_API bool IsArrayType(const FName& DataType, UPARAM(DisplayName = "NOT") bool bInvert = false);

	// Returns true if the given MetaSound data type is registered and can be used for constructor pins.
	UFUNCTION(BlueprintCallable, BlueprintPure, DisplayName = "Is MetaSound Constructor Type", Category = "TechAudioTools")
	static TECHAUDIOTOOLSMETASOUND_API bool IsConstructorType(const FName& DataType, UPARAM(DisplayName = "NOT") bool bInvert = false);
};
