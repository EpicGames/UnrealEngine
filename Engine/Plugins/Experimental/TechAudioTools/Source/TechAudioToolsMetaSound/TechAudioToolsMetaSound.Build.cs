// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class TechAudioToolsMetaSound : ModuleRules
{
    public TechAudioToolsMetaSound(ReadOnlyTargetRules Target) : base(Target)
    {
        PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

        PublicDependencyModuleNames.AddRange(
            new string[]
            {
                "Core",
				"MetasoundEngine",
				"MetasoundFrontend",
				"ModelViewViewModel",
				"SlateCore",
				"TechAudioTools",
            }
        );

        PrivateDependencyModuleNames.AddRange(
            new string[]
            {
	            "AudioExtensions",
                "CoreUObject",
                "Engine",
                "Slate",
                "UMG",
            }
        );
    }
}
