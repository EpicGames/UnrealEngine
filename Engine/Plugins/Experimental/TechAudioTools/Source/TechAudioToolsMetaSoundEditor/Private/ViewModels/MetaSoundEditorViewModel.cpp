// Copyright Epic Games, Inc. All Rights Reserved.

#include "ViewModels/MetaSoundEditorViewModel.h"

#include "MetasoundBuilderSubsystem.h"
#include "Editor/EditorEngine.h"
#include "Logging/StructuredLog.h"
#include "MetasoundDocumentBuilderRegistry.h"
#include "MetasoundEditorBuilderListener.h"
#include "MetasoundEditorSubsystem.h"
#include "MetasoundOutputSubsystem.h"
#include "TechAudioToolsLog.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MetaSoundEditorViewModel)

extern UNREALED_API UEditorEngine* GEditor;

void UMetaSoundEditorViewModel::InitializeMetaSound(const TScriptInterface<IMetaSoundDocumentInterface> InMetaSound)
{
	if (!InMetaSound)
	{
		Reset();
		return;
	}

	Builder = &Metasound::Engine::FDocumentBuilderRegistry::GetChecked().FindOrBeginBuilding(*InMetaSound.GetObject(), true);
	Initialize(Builder);
}

void UMetaSoundEditorViewModel::Initialize(UMetaSoundBuilderBase* InBuilder)
{
	Reset();

	Super::Initialize(InBuilder);

	if (!InBuilder)
	{
		return;
	}

	const FMetaSoundFrontendDocumentBuilder& DocumentBuilder = InBuilder->GetConstBuilder();
	const FMetasoundFrontendDocument& FrontendDocument = DocumentBuilder.GetConstDocumentChecked();

	OnDisplayNameChanged(FrontendDocument.RootGraph.Metadata.GetDisplayName());
	OnDescriptionChanged(FrontendDocument.RootGraph.Metadata.GetDescription());
	OnAuthorChanged(FrontendDocument.RootGraph.Metadata.GetAuthor());
	OnKeywordsChanged(FrontendDocument.RootGraph.Metadata.GetKeywords());
	OnCategoryHierarchyChanged(FrontendDocument.RootGraph.Metadata.GetCategoryHierarchy());
	OnIsDeprecatedChanged(EnumHasAllFlags(FrontendDocument.RootGraph.Metadata.GetAccessFlags(), EMetasoundFrontendClassAccessFlags::Deprecated));

	BindDelegates();
}

void UMetaSoundEditorViewModel::Reset()
{
	Super::Reset();

	UE_MVVM_SET_PROPERTY_VALUE(DisplayName, FText());
	UE_MVVM_SET_PROPERTY_VALUE(Description, FText());
	UE_MVVM_SET_PROPERTY_VALUE(Author, FString());
	UE_MVVM_SET_PROPERTY_VALUE(Keywords, TArray<FText>());
	UE_MVVM_SET_PROPERTY_VALUE(CategoryHierarchy, TArray<FText>());
	UE_MVVM_SET_PROPERTY_VALUE(bIsDeprecated, false);

	UnbindDelegates();
}

void UMetaSoundEditorViewModel::SetMetaSoundDisplayName(const FText& InDisplayName)
{
	if (Builder && UE_MVVM_SET_PROPERTY_VALUE(DisplayName, InDisplayName))
	{
		FMetaSoundFrontendDocumentBuilder& DocumentBuilder = Builder->GetBuilder();
		DocumentBuilder.SetDisplayName(InDisplayName);
	}
}

void UMetaSoundEditorViewModel::SetMetaSoundDescription(const FText& InDescription)
{
	if (Builder && UE_MVVM_SET_PROPERTY_VALUE(Description, InDescription))
	{
		FMetaSoundFrontendDocumentBuilder& DocumentBuilder = Builder->GetBuilder();
		DocumentBuilder.SetDescription(InDescription);
	}
}

void UMetaSoundEditorViewModel::SetAuthor(const FString& InAuthor)
{
	if (Builder && UE_MVVM_SET_PROPERTY_VALUE(Author, InAuthor))
	{
		FMetaSoundFrontendDocumentBuilder& DocumentBuilder = Builder->GetBuilder();
		DocumentBuilder.SetAuthor(InAuthor);
	}
}

void UMetaSoundEditorViewModel::SetKeywords(const TArray<FText>& InKeywords)
{
	if (Builder && UE_MVVM_SET_PROPERTY_VALUE(Keywords, InKeywords))
	{
		FMetaSoundFrontendDocumentBuilder& DocumentBuilder = Builder->GetBuilder();
		DocumentBuilder.SetKeywords(InKeywords);
	}
}

void UMetaSoundEditorViewModel::SetCategoryHierarchy(const TArray<FText>& InCategoryHierarchy)
{
	if (Builder && UE_MVVM_SET_PROPERTY_VALUE(CategoryHierarchy, InCategoryHierarchy))
	{
		FMetaSoundFrontendDocumentBuilder& DocumentBuilder = Builder->GetBuilder();
		DocumentBuilder.SetCategoryHierarchy(InCategoryHierarchy);
	}
}

void UMetaSoundEditorViewModel::SetIsDeprecated(const bool bInIsDeprecated)
{
	if (Builder && UE_MVVM_SET_PROPERTY_VALUE(bIsDeprecated, bInIsDeprecated))
	{
		FMetaSoundFrontendDocumentBuilder& DocumentBuilder = Builder->GetBuilder();
		if (bInIsDeprecated)
		{
			DocumentBuilder.AddAccessFlags(EMetasoundFrontendClassAccessFlags::Deprecated);
		}
		else
		{
			DocumentBuilder.RemoveAccessFlags(EMetasoundFrontendClassAccessFlags::Deprecated);
		}
	}
}

void UMetaSoundEditorViewModel::SetInputDisplayName(const FName& InputName, const FText& InDisplayName) const
{
	if (Builder)
	{
		FMetaSoundFrontendDocumentBuilder& DocumentBuilder = Builder->GetBuilder();
		DocumentBuilder.SetGraphInputDisplayName(InputName, InDisplayName);
	}
}

void UMetaSoundEditorViewModel::SetInputDescription(const FName& InputName, const FText& InDescription) const
{
	if (Builder)
	{
		FMetaSoundFrontendDocumentBuilder& DocumentBuilder = Builder->GetBuilder();
		DocumentBuilder.SetGraphInputDescription(InputName, InDescription);
	}
}

void UMetaSoundEditorViewModel::SetInputSortOrderIndex(const FName& InputName, const int32 InSortOrderIndex) const
{
	if (Builder)
	{
		FMetaSoundFrontendDocumentBuilder& DocumentBuilder = Builder->GetBuilder();
		DocumentBuilder.SetGraphInputSortOrderIndex(InputName, InSortOrderIndex);
	}
}

void UMetaSoundEditorViewModel::SetInputIsAdvancedDisplay(const FName& InputName, const bool bInIsAdvancedDisplay) const
{
	if (Builder)
	{
		FMetaSoundFrontendDocumentBuilder& DocumentBuilder = Builder->GetBuilder();
		DocumentBuilder.SetGraphInputAdvancedDisplay(InputName, bInIsAdvancedDisplay);
	}
}

void UMetaSoundEditorViewModel::SetOutputDisplayName(const FName& OutputName, const FText& InDisplayName) const
{
	if (Builder)
	{
		FMetaSoundFrontendDocumentBuilder& DocumentBuilder = Builder->GetBuilder();
		DocumentBuilder.SetGraphOutputDisplayName(OutputName, InDisplayName);
	}
}

void UMetaSoundEditorViewModel::SetOutputDescription(const FName& OutputName, const FText& InDescription) const
{
	if (Builder)
	{
		FMetaSoundFrontendDocumentBuilder& DocumentBuilder = Builder->GetBuilder();
		DocumentBuilder.SetGraphOutputDescription(OutputName, InDescription);
	}
}

void UMetaSoundEditorViewModel::SetOutputSortOrderIndex(const FName& OutputName, const int32 InSortOrderIndex) const
{
	if (Builder)
	{
		FMetaSoundFrontendDocumentBuilder& DocumentBuilder = Builder->GetBuilder();
		DocumentBuilder.SetGraphOutputSortOrderIndex(OutputName, InSortOrderIndex);
	}
}

void UMetaSoundEditorViewModel::SetOutputIsAdvancedDisplay(const FName& OutputName, const bool bInIsAdvancedDisplay) const
{
	if (Builder)
	{
		FMetaSoundFrontendDocumentBuilder& DocumentBuilder = Builder->GetBuilder();
		DocumentBuilder.SetGraphOutputAdvancedDisplay(OutputName, bInIsAdvancedDisplay);
	}
}

void UMetaSoundEditorViewModel::OnGraphInputDefaultChanged(const FGraphInputDefaultChangedArgs& Args)
{
	if (UMetaSoundInputViewModel* InputViewModel = InputViewModels.FindRef(Args.InputName))
	{
		InputViewModel->OnInputDefaultChanged(Args.Literal, Args.PageName, Args.ChangeType);
	}
}

void UMetaSoundEditorViewModel::OnInputDisplayNameChanged(const FName InputName, const FText InDisplayName)
{
	if (UMetaSoundInputEditorViewModel* InputEditorViewModel = Cast<UMetaSoundInputEditorViewModel>(InputViewModels.FindRef(InputName)))
	{
		InputEditorViewModel->OnInputDisplayNameChanged(InDisplayName);
	}
}

void UMetaSoundEditorViewModel::OnInputDescriptionChanged(const FName InputName, FText InDescription)
{
	if (UMetaSoundInputEditorViewModel* InputEditorViewModel = Cast<UMetaSoundInputEditorViewModel>(InputViewModels.FindRef(InputName)))
	{
		InputEditorViewModel->OnInputDescriptionChanged(InDescription);
	}
}

void UMetaSoundEditorViewModel::OnInputSortOrderIndexChanged(const FName InputName, const int32 InSortOrderIndex)
{
	if (UMetaSoundInputEditorViewModel* InputEditorViewModel = Cast<UMetaSoundInputEditorViewModel>(InputViewModels.FindRef(InputName)))
	{
		InputEditorViewModel->OnInputSortOrderIndexChanged(InSortOrderIndex);
	}
}

void UMetaSoundEditorViewModel::OnInputIsAdvancedDisplayChanged(const FName InputName, const bool bInIsAdvancedDisplay)
{
	if (UMetaSoundInputEditorViewModel* InputEditorViewModel = Cast<UMetaSoundInputEditorViewModel>(InputViewModels.FindRef(InputName)))
	{
		InputEditorViewModel->OnInputIsAdvancedDisplayChanged(bInIsAdvancedDisplay);
	}
}

void UMetaSoundEditorViewModel::OnOutputDisplayNameChanged(const FName OutputName, const FText NewName)
{
	if (UMetaSoundOutputEditorViewModel* OutputEditorViewModel = Cast<UMetaSoundOutputEditorViewModel>(OutputViewModels.FindRef(OutputName)))
	{
		OutputEditorViewModel->OnOutputDisplayNameChanged(NewName);
	}
}

void UMetaSoundEditorViewModel::OnOutputDescriptionChanged(const FName OutputName, const FText InDescription)
{
	if (UMetaSoundOutputEditorViewModel* OutputEditorViewModel = Cast<UMetaSoundOutputEditorViewModel>(OutputViewModels.FindRef(OutputName)))
	{
		OutputEditorViewModel->OnOutputDescriptionChanged(InDescription);
	}
}

void UMetaSoundEditorViewModel::OnOutputSortOrderIndexChanged(const FName OutputName, const int32 InSortOrderIndex)
{
	if (UMetaSoundOutputEditorViewModel* OutputEditorViewModel = Cast<UMetaSoundOutputEditorViewModel>(OutputViewModels.FindRef(OutputName)))
	{
		OutputEditorViewModel->OnOutputSortOrderIndexChanged(InSortOrderIndex);
	}
}

void UMetaSoundEditorViewModel::OnOutputIsAdvancedDisplayChanged(const FName OutputName, const bool bInIsAdvancedDisplay)
{
	if (UMetaSoundOutputEditorViewModel* OutputEditorViewModel = Cast<UMetaSoundOutputEditorViewModel>(OutputViewModels.FindRef(OutputName)))
	{
		OutputEditorViewModel->OnOutputIsAdvancedDisplayChanged(bInIsAdvancedDisplay);
	}
}

UMetaSoundInputViewModel* UMetaSoundEditorViewModel::CreateInputViewModel(const FMetasoundFrontendClassInput& InInput)
{
	if (UMetaSoundInputEditorViewModel* InputEditorViewModel = Cast<UMetaSoundInputEditorViewModel>(Super::CreateInputViewModel(InInput)))
	{
		InputEditorViewModel->OnInputDisplayNameChanged(InInput.Metadata.GetDisplayName());
		InputEditorViewModel->OnInputDescriptionChanged(InInput.Metadata.GetDescription());
		InputEditorViewModel->OnInputSortOrderIndexChanged(InInput.Metadata.SortOrderIndex);
		InputEditorViewModel->OnInputIsAdvancedDisplayChanged(InInput.Metadata.bIsAdvancedDisplay);

		return InputEditorViewModel;
	}

	return nullptr;
}

UMetaSoundOutputViewModel* UMetaSoundEditorViewModel::CreateOutputViewModel(const FMetasoundFrontendClassOutput& InOutput)
{
	if (UMetaSoundOutputEditorViewModel* OutputEditorViewModel = Cast<UMetaSoundOutputEditorViewModel>(Super::CreateOutputViewModel(InOutput)))
	{
		OutputEditorViewModel->OnOutputDisplayNameChanged(InOutput.Metadata.GetDisplayName());
		OutputEditorViewModel->OnOutputDescriptionChanged(InOutput.Metadata.GetDescription());
		OutputEditorViewModel->OnOutputSortOrderIndexChanged(InOutput.Metadata.SortOrderIndex);
		OutputEditorViewModel->OnOutputIsAdvancedDisplayChanged(InOutput.Metadata.bIsAdvancedDisplay);

		return OutputEditorViewModel;
	}

	return nullptr;
}

TSubclassOf<UMetaSoundInputViewModel> UMetaSoundEditorViewModel::GetInputViewModelClass() const
{
	return UMetaSoundInputEditorViewModel::StaticClass();
}

TSubclassOf<UMetaSoundOutputViewModel> UMetaSoundEditorViewModel::GetOutputViewModelClass() const
{
	return UMetaSoundOutputEditorViewModel::StaticClass();
}

void UMetaSoundEditorViewModel::BindDelegates()
{
	if (!Builder)
	{
		UE_LOGFMT(LogTechAudioTools, Log, "Could not bind MetaSoundViewModel delegates. Builder was null.");
		SetIsInitialized(false);
		return;
	}

	UMetaSoundEditorSubsystem* EditorSubsystem = GEditor->GetEditorSubsystem<UMetaSoundEditorSubsystem>();
	if (!EditorSubsystem)
	{
		UE_LOGFMT(LogTechAudioTools, Log, "Could not bind MetaSoundViewModel delegates. Unable to locate MetaSound Editor Subsystem.");
		SetIsInitialized(false);
		return;
	}

	EMetaSoundBuilderResult Result;
	BuilderListener = EditorSubsystem->AddBuilderDelegateListener(Builder, Result);

	if (Result != EMetaSoundBuilderResult::Succeeded)
	{
		UE_LOGFMT(LogTechAudioTools, Warning, "Could not bind MetaSoundViewModel delegates. Failed to create BuilderListener.");
		SetIsInitialized(false);
		return;
	}

	if (BuilderListener)
	{
		BuilderListener->OnGraphInputAddedDelegate.AddDynamic(this, &UMetaSoundViewModel::OnInputAdded);
		BuilderListener->OnRemovingGraphInputDelegate.AddDynamic(this, &UMetaSoundViewModel::OnInputRemoved);
		BuilderListener->OnGraphInputNameChangedDelegate.AddDynamic(this, &UMetaSoundViewModel::OnInputNameChanged);
		BuilderListener->OnGraphInputDataTypeChangedDelegate.AddDynamic(this, &UMetaSoundViewModel::OnInputDataTypeChanged);
		BuilderListener->OnGraphInputDefaultChangedWithArgsDelegate.AddDynamic(this, &ThisClass::OnGraphInputDefaultChanged);
		BuilderListener->OnGraphInputInheritsDefaultChangedDelegate.AddDynamic(this, &UMetaSoundViewModel::OnInputInheritsDefaultChanged);
		BuilderListener->OnGraphInputIsConstructorPinChangedDelegate.AddDynamic(this, &UMetaSoundViewModel::OnInputIsConstructorPinChanged);

		BuilderListener->OnGraphOutputAddedDelegate.AddDynamic(this, &UMetaSoundViewModel::OnOutputAdded);
		BuilderListener->OnRemovingGraphOutputDelegate.AddDynamic(this, &UMetaSoundViewModel::OnOutputRemoved);
		BuilderListener->OnGraphOutputNameChangedDelegate.AddDynamic(this, &UMetaSoundViewModel::OnOutputNameChanged);
		BuilderListener->OnGraphOutputDataTypeChangedDelegate.AddDynamic(this, &UMetaSoundViewModel::OnOutputDataTypeChanged);
		BuilderListener->OnGraphOutputIsConstructorPinChangedDelegate.AddDynamic(this, &UMetaSoundViewModel::OnOutputIsConstructorPinChanged);

		// Note: Document metadata delegates currently share a single delegate in the builder listener which is called when any metadata changes, which calls all of these together.
		BuilderListener->OnDocumentDisplayNameChangedDelegate.AddDynamic(this, &ThisClass::OnDisplayNameChanged);
		BuilderListener->OnDocumentDescriptionChangedDelegate.AddDynamic(this, &ThisClass::OnDescriptionChanged);
		BuilderListener->OnDocumentAuthorChangedDelegate.AddDynamic(this, &ThisClass::OnAuthorChanged);
		BuilderListener->OnDocumentKeywordsChangedDelegate.AddDynamic(this, &ThisClass::OnKeywordsChanged);
		BuilderListener->OnDocumentCategoryHierarchyChangedDelegate.AddDynamic(this, &ThisClass::OnCategoryHierarchyChanged);
		BuilderListener->OnDocumentIsDeprecatedChangedDelegate.AddDynamic(this, &ThisClass::OnIsDeprecatedChanged);

		BuilderListener->OnGraphInputDisplayNameChangedDelegate.AddDynamic(this, &ThisClass::OnInputDisplayNameChanged);
		BuilderListener->OnGraphInputDescriptionChangedDelegate.AddDynamic(this, &ThisClass::OnInputDescriptionChanged);
		BuilderListener->OnGraphInputSortOrderIndexChangedDelegate.AddDynamic(this, &ThisClass::OnInputSortOrderIndexChanged);
		BuilderListener->OnGraphInputIsAdvancedDisplayChangedDelegate.AddDynamic(this, &ThisClass::OnInputIsAdvancedDisplayChanged);

		BuilderListener->OnGraphOutputDisplayNameChangedDelegate.AddDynamic(this, &ThisClass::OnOutputDisplayNameChanged);
		BuilderListener->OnGraphOutputDescriptionChangedDelegate.AddDynamic(this, &ThisClass::OnOutputDescriptionChanged);
		BuilderListener->OnGraphOutputSortOrderIndexChangedDelegate.AddDynamic(this, &ThisClass::OnOutputSortOrderIndexChanged);
		BuilderListener->OnGraphOutputIsAdvancedDisplayChangedDelegate.AddDynamic(this, &ThisClass::OnOutputIsAdvancedDisplayChanged);
	}
}

void UMetaSoundEditorViewModel::UnbindDelegates()
{
	if (BuilderListener)
	{
		BuilderListener->RemoveAllDelegates();
		BuilderListener = nullptr;
	}
}

void UMetaSoundInputEditorViewModel::SetInputDisplayName(const FText& InDisplayName)
{
	if (UE_MVVM_SET_PROPERTY_VALUE(InputDisplayName, InDisplayName))
	{
		if (const UMetaSoundEditorViewModel* MetaSoundEditorViewModel = Cast<UMetaSoundEditorViewModel>(GetOuter()))
		{
			MetaSoundEditorViewModel->SetInputDisplayName(InputName, InDisplayName);
		}
	}
}

void UMetaSoundInputEditorViewModel::SetInputDescription(const FText& InDescription)
{
	if (UE_MVVM_SET_PROPERTY_VALUE(InputDescription, InDescription))
	{
		if (const UMetaSoundEditorViewModel* MetaSoundEditorViewModel = Cast<UMetaSoundEditorViewModel>(GetOuter()))
		{
			MetaSoundEditorViewModel->SetInputDescription(InputName, InDescription);
		}
	}
}

void UMetaSoundInputEditorViewModel::SetSortOrderIndex(const int32 InSortOrderIndex)
{
	if (UE_MVVM_SET_PROPERTY_VALUE(SortOrderIndex, InSortOrderIndex))
	{
		if (const UMetaSoundEditorViewModel* MetaSoundEditorViewModel = Cast<UMetaSoundEditorViewModel>(GetOuter()))
		{
			MetaSoundEditorViewModel->SetInputSortOrderIndex(InputName, InSortOrderIndex);
		}
	}
}

void UMetaSoundInputEditorViewModel::SetIsAdvancedDisplay(const bool bInIsAdvancedDisplay)
{
	if (UE_MVVM_SET_PROPERTY_VALUE(bIsAdvancedDisplay, bInIsAdvancedDisplay))
	{
		if (const UMetaSoundEditorViewModel* MetaSoundEditorViewModel = Cast<UMetaSoundEditorViewModel>(GetOuter()))
		{
			MetaSoundEditorViewModel->SetInputIsAdvancedDisplay(InputName, bInIsAdvancedDisplay);
		}
	}
}

void UMetaSoundOutputEditorViewModel::SetOutputDisplayName(const FText& InDisplayName)
{
	if (UE_MVVM_SET_PROPERTY_VALUE(OutputDisplayName, InDisplayName))
	{
		if (const UMetaSoundEditorViewModel* MetaSoundEditorViewModel = Cast<UMetaSoundEditorViewModel>(GetOuter()))
		{
			MetaSoundEditorViewModel->SetOutputDisplayName(OutputName, InDisplayName);
		}
	}
}

void UMetaSoundOutputEditorViewModel::SetOutputDescription(const FText& InDescription)
{
	if (UE_MVVM_SET_PROPERTY_VALUE(OutputDescription, InDescription))
	{
		if (const UMetaSoundEditorViewModel* MetaSoundEditorViewModel = Cast<UMetaSoundEditorViewModel>(GetOuter()))
		{
			MetaSoundEditorViewModel->SetOutputDescription(OutputName, InDescription);
		}
	}
}

void UMetaSoundOutputEditorViewModel::SetSortOrderIndex(const int32 InSortOrderIndex)
{
	if (UE_MVVM_SET_PROPERTY_VALUE(SortOrderIndex, InSortOrderIndex))
	{
		if (const UMetaSoundEditorViewModel* MetaSoundEditorViewModel = Cast<UMetaSoundEditorViewModel>(GetOuter()))
		{
			MetaSoundEditorViewModel->SetOutputSortOrderIndex(OutputName, InSortOrderIndex);
		}
	}
}

void UMetaSoundOutputEditorViewModel::SetIsAdvancedDisplay(const bool bInIsAdvancedDisplay)
{
	if (UE_MVVM_SET_PROPERTY_VALUE(bIsAdvancedDisplay, bInIsAdvancedDisplay))
	{
		if (const UMetaSoundEditorViewModel* MetaSoundEditorViewModel = Cast<UMetaSoundEditorViewModel>(GetOuter()))
		{
			MetaSoundEditorViewModel->SetOutputIsAdvancedDisplay(OutputName, bInIsAdvancedDisplay);
		}
	}
}
