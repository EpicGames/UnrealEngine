// Copyright Epic Games, Inc. All Rights Reserved.

#include "ViewModels/MetaSoundEditorViewModelConversionFunctions.h"

#include "Editor/EditorEngine.h"
#include "IMetaSoundGraphPanelPinFactory.h"
#include "MetasoundEditorGraphSchema.h"
#include "MetasoundEditorModule.h"
#include "Modules/ModuleManager.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MetaSoundEditorViewModelConversionFunctions)

extern UNREALED_API UEditorEngine* GEditor;

bool UMetaSoundEditorViewModelConversionFunctions::IsEditorOnly() const
{
	return true;
}

UWorld* UMetaSoundEditorViewModelConversionFunctions::GetWorld() const
{
	if (HasAllFlags(RF_ClassDefaultObject))
	{
		return nullptr;
	}

	return GEditor ? GEditor->GetEditorWorldContext(false).World() : nullptr;
}

FLinearColor UMetaSoundEditorViewModelConversionFunctions::GetMetaSoundDataTypePinColor(const FName& DataType)
{
	using namespace Metasound::Editor;

	IMetasoundEditorModule& MetaSoundEditorModule = FModuleManager::GetModuleChecked<IMetasoundEditorModule>(IMetasoundEditorModule::ModuleName);
	if (const FEdGraphPinType* PinType = MetaSoundEditorModule.GetGraphPanelPinFactory()->FindPinType(DataType))
	{
		if (const UMetasoundEditorGraphSchema* Schema = GetDefault<UMetasoundEditorGraphSchema>())
		{
			return Schema->GetPinTypeColor(*PinType);
		}
	}

	return FLinearColor::Black;
}
