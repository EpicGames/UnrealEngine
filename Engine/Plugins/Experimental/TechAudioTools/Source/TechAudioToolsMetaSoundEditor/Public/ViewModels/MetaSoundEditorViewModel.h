// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ViewModels/MetaSoundViewModel.h"

#include "MetaSoundEditorViewModel.generated.h"

#define UE_API TECHAUDIOTOOLSMETASOUNDEDITOR_API

struct FGraphInputDefaultChangedArgs;
class UMetaSoundEditorBuilderListener;

/**
 * Editor viewmodel for MetaSounds. Creates MetaSoundEditorBuilderListener bindings upon initialization, allowing changes made to assets in the
 * MetaSound Editor to be reflected in UMG widgets.
 */
UCLASS(MinimalAPI, DisplayName = "MetaSound Editor Viewmodel")
class UMetaSoundEditorViewModel : public UMetaSoundViewModel
{
	GENERATED_BODY()

protected:
	// Sets the display name of the initialized MetaSound.
	UPROPERTY(BlueprintReadWrite, FieldNotify, Getter, Setter = "SetMetaSoundDisplayName", Category = "MetaSound Viewmodel|Metadata", meta = (AllowPrivateAccess))
	FText DisplayName;

	// Sets the description of the initialized MetaSound.
	UPROPERTY(BlueprintReadWrite, FieldNotify, Getter, Setter = "SetMetaSoundDescription", Category = "MetaSound Viewmodel|Metadata", meta = (AllowPrivateAccess))
	FText Description;

	// Sets the author of the initialized MetaSound.
	UPROPERTY(BlueprintReadWrite, FieldNotify, Getter, Setter, Category = "MetaSound Viewmodel|Metadata", meta = (AllowPrivateAccess))
	FString Author;

	// Sets the keywords of the initialized MetaSound.
	UPROPERTY(BlueprintReadWrite, FieldNotify, Getter, Setter, Category = "MetaSound Viewmodel|Metadata", meta = (AllowPrivateAccess))
	TArray<FText> Keywords;

	// Sets the category hierarchy of the initialized MetaSound.
	UPROPERTY(BlueprintReadWrite, FieldNotify, Getter, Setter, Category = "MetaSound Viewmodel|Metadata", meta = (AllowPrivateAccess))
	TArray<FText> CategoryHierarchy;

	// Sets the initialized MetaSound asset as deprecated.
	UPROPERTY(BlueprintReadWrite, FieldNotify, Getter = "IsDeprecated", Setter = "SetIsDeprecated", Category = "MetaSound Viewmodel|Metadata", meta = (AllowPrivateAccess))
	bool bIsDeprecated = false;

public:
	virtual bool IsEditorOnly() const override { return true; }

	virtual void InitializeMetaSound(const TScriptInterface<IMetaSoundDocumentInterface> InMetaSound) override;
	virtual void Initialize(UMetaSoundBuilderBase* InBuilder) override;
	virtual void Reset() override;

	FText GetDisplayName() const { return DisplayName; }
	FText GetDescription() const { return Description; };
	FString GetAuthor() const { return Author; };
	TArray<FText> GetKeywords() const { return Keywords; };
	TArray<FText> GetCategoryHierarchy() const { return CategoryHierarchy; };
	bool IsDeprecated() const { return bIsDeprecated; };

	UE_API void SetMetaSoundDisplayName(const FText& InDisplayName);
	UE_API void SetMetaSoundDescription(const FText& InDescription);
	UE_API void SetAuthor(const FString& InAuthor);
	UE_API void SetKeywords(const TArray<FText>& InKeywords);
	UE_API void SetCategoryHierarchy(const TArray<FText>& InCategoryHierarchy);
	UE_API void SetIsDeprecated(bool bInIsDeprecated);

	UE_API void SetInputDisplayName(const FName& InputName, const FText& InDisplayName) const;
	UE_API void SetInputDescription(const FName& InputName, const FText& InDescription) const;
	UE_API void SetInputSortOrderIndex(const FName& InputName, int32 InSortOrderIndex) const;
	UE_API void SetInputIsAdvancedDisplay(const FName& InputName, bool bInIsAdvancedDisplay) const;

	UE_API void SetOutputDisplayName(const FName& OutputName, const FText& InDisplayName) const;
	UE_API void SetOutputDescription(const FName& OutputName, const FText& InDescription) const;
	UE_API void SetOutputSortOrderIndex(const FName& OutputName, int32 InSortOrderIndex) const;
	UE_API void SetOutputIsAdvancedDisplay(const FName& OutputName, bool bInIsAdvancedDisplay) const;

	UFUNCTION()
	void OnGraphInputDefaultChanged(const FGraphInputDefaultChangedArgs& Args);
	
	UFUNCTION()
	void OnDisplayNameChanged(const FText NewDisplayName) { UE_MVVM_SET_PROPERTY_VALUE(DisplayName, NewDisplayName); }
	UFUNCTION()
	void OnDescriptionChanged(const FText NewDescription) { UE_MVVM_SET_PROPERTY_VALUE(Description, NewDescription); }
	UFUNCTION()
	void OnAuthorChanged(const FString NewAuthor) { UE_MVVM_SET_PROPERTY_VALUE(Author, NewAuthor); }
	UFUNCTION()
	void OnKeywordsChanged(const TArray<FText> NewKeywords) { UE_MVVM_SET_PROPERTY_VALUE(Keywords, NewKeywords); }
	UFUNCTION()
	void OnCategoryHierarchyChanged(const TArray<FText> NewCategoryHierarchy) { UE_MVVM_SET_PROPERTY_VALUE(CategoryHierarchy, NewCategoryHierarchy); }
	UFUNCTION()
	void OnIsDeprecatedChanged(const bool bNewIsDeprecated) { UE_MVVM_SET_PROPERTY_VALUE(bIsDeprecated, bNewIsDeprecated); }

	UFUNCTION()
	void OnInputDisplayNameChanged(FName InputName, FText InDisplayName);
	UFUNCTION()
	void OnInputDescriptionChanged(FName InputName, FText InDescription);
	UFUNCTION()
	void OnInputSortOrderIndexChanged(FName InputName, int32 InSortOrderIndex);
	UFUNCTION()
	void OnInputIsAdvancedDisplayChanged(FName InputName, bool bInIsAdvancedDisplay);

	UFUNCTION()
	void OnOutputDisplayNameChanged(FName OutputName, const FText InDisplayName);
	UFUNCTION()
	void OnOutputDescriptionChanged(FName OutputName, const FText InDescription);
	UFUNCTION()
	void OnOutputSortOrderIndexChanged(FName OutputName, int32 InSortOrderIndex);
	UFUNCTION()
	void OnOutputIsAdvancedDisplayChanged(FName OutputName, bool bInIsAdvancedDisplay);

protected:
	UPROPERTY(Transient)
	TObjectPtr<UMetaSoundEditorBuilderListener> BuilderListener;

	virtual UMetaSoundInputViewModel* CreateInputViewModel(const FMetasoundFrontendClassInput& InInput) override;
	virtual UMetaSoundOutputViewModel* CreateOutputViewModel(const FMetasoundFrontendClassOutput& InOutput) override;

	virtual TSubclassOf<UMetaSoundInputViewModel> GetInputViewModelClass() const override;
	virtual TSubclassOf<UMetaSoundOutputViewModel> GetOutputViewModelClass() const override;

	void BindDelegates();
	void UnbindDelegates();
};

/**
 * Editor viewmodel class for MetaSound inputs. Extends the runtime MetaSoundInputViewModel with editor-only functionality. 
 */
UCLASS(MinimalAPI, DisplayName = "MetaSound Input Editor Viewmodel")
class UMetaSoundInputEditorViewModel : public UMetaSoundInputViewModel
{
	GENERATED_BODY()

protected:
	// Sets the display name of the initialized MetaSound input.
	UPROPERTY(BlueprintReadWrite, FieldNotify, Setter, Category = "MetaSound Input", meta = (AllowPrivateAccess))
	FText InputDisplayName;

	// Sets the description of the initialized MetaSound input.
	UPROPERTY(BlueprintReadWrite, FieldNotify, Setter, Category = "MetaSound Input", meta = (AllowPrivateAccess))
	FText InputDescription;

	// Sets the sort order index of the initialized MetaSound input.
	UPROPERTY(BlueprintReadWrite, FieldNotify, Setter, Category = "MetaSound Input", meta = (AllowPrivateAccess))
	int32 SortOrderIndex;

	// Sets whether the initialized MetaSound input should be located in the Advanced Display category.
	UPROPERTY(BlueprintReadWrite, FieldNotify, Setter = "SetIsAdvancedDisplay", Category = "MetaSound Input", meta = (AllowPrivateAccess))
	bool bIsAdvancedDisplay;

public:
	virtual bool IsEditorOnly() const override { return true; }

	FText GetInputDisplayName() const { return InputDisplayName; }
	FText GetInputDescription() const { return InputDescription; }
	int32 GetSortOrderIndex() const { return SortOrderIndex; }
	bool IsAdvancedDisplay() const { return bIsAdvancedDisplay; }

	UE_API void SetInputDisplayName(const FText& InDisplayName);
	UE_API void SetInputDescription(const FText& InDescription);
	UE_API void SetSortOrderIndex(int32 InSortOrderIndex);
	UE_API void SetIsAdvancedDisplay(bool bInIsAdvancedDisplay);

	void OnInputDisplayNameChanged(const FText& NewDisplayName) { UE_MVVM_SET_PROPERTY_VALUE(InputDisplayName, NewDisplayName); }
	void OnInputDescriptionChanged(const FText& NewDescription) { UE_MVVM_SET_PROPERTY_VALUE(InputDescription, NewDescription); }
	void OnInputSortOrderIndexChanged(const int32 NewSortOrderIndex) { UE_MVVM_SET_PROPERTY_VALUE(SortOrderIndex, NewSortOrderIndex); }
	void OnInputIsAdvancedDisplayChanged(const bool bNewIsAdvancedDisplay) { UE_MVVM_SET_PROPERTY_VALUE(bIsAdvancedDisplay, bNewIsAdvancedDisplay); }
};

/**
 * Editor viewmodel class for MetaSound outputs. Extends the runtime MetaSoundOutputViewModel with editor-only functionality. 
 */
UCLASS(MinimalAPI, DisplayName = "MetaSound Output Editor Viewmodel")
class UMetaSoundOutputEditorViewModel : public UMetaSoundOutputViewModel
{
	GENERATED_BODY()

protected:
	// Sets the display name of the initialized MetaSound output.
	UPROPERTY(BlueprintReadWrite, FieldNotify, Setter, Category = "MetaSound Output", meta = (AllowPrivateAccess))
	FText OutputDisplayName;

	// Sets the description of the initialized MetaSound output.
	UPROPERTY(BlueprintReadWrite, FieldNotify, Setter, Category = "MetaSound Output", meta = (AllowPrivateAccess))
	FText OutputDescription;

	// Sets the sort order index of the initialized MetaSound output.
	UPROPERTY(BlueprintReadWrite, FieldNotify, Setter, Category = "MetaSound Output", meta = (AllowPrivateAccess))
	int32 SortOrderIndex;

	// Sets whether the initialized MetaSound output should be located in the Advanced Display category.
	UPROPERTY(BlueprintReadWrite, FieldNotify, Setter = "SetIsAdvancedDisplay", Category = "MetaSound Output", meta = (AllowPrivateAccess))
	bool bIsAdvancedDisplay;

public:
	virtual bool IsEditorOnly() const override { return true; }

	FText GetOutputDisplayName() const { return OutputDisplayName; }
	FText GetOutputDescription() const { return OutputDescription; }
	int32 GetSortOrderIndex() const { return SortOrderIndex; }
	bool IsAdvancedDisplay() const { return bIsAdvancedDisplay; }

	UE_API void SetOutputDisplayName(const FText& InDisplayName);
	UE_API void SetOutputDescription(const FText& InDescription);
	UE_API void SetSortOrderIndex(int32 InSortOrderIndex);
	UE_API void SetIsAdvancedDisplay(bool bInIsAdvancedDisplay);

	void OnOutputDisplayNameChanged(const FText& NewDisplayName) { UE_MVVM_SET_PROPERTY_VALUE(OutputDisplayName, NewDisplayName); }
	void OnOutputDescriptionChanged(const FText& NewDescription) { UE_MVVM_SET_PROPERTY_VALUE(OutputDescription, NewDescription); }
	void OnOutputSortOrderIndexChanged(const int32 NewSortOrderIndex) { UE_MVVM_SET_PROPERTY_VALUE(SortOrderIndex, NewSortOrderIndex); }
	void OnOutputIsAdvancedDisplayChanged(const bool NewIsAdvancedDisplay) { UE_MVVM_SET_PROPERTY_VALUE(bIsAdvancedDisplay, NewIsAdvancedDisplay); }
};

#undef UE_API
