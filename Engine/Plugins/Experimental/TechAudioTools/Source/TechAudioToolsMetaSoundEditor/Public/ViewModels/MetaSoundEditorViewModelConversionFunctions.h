// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Kismet/BlueprintFunctionLibrary.h"

#include "MetaSoundEditorViewModelConversionFunctions.generated.h"

/**
 * Collection of conversion functions to use with MetaSound Editor Viewmodels.
 */
UCLASS(MinimalAPI)
class UMetaSoundEditorViewModelConversionFunctions : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:
	virtual bool IsEditorOnly() const override;
	virtual UWorld* GetWorld() const override;

	// Returns the pin color associated with the given data type.
	UFUNCTION(BlueprintCallable, BlueprintPure, DisplayName = "Get MetaSound Data Type Pin Color", Category = "TechAudioTools")
	static FLinearColor GetMetaSoundDataTypePinColor(const FName& DataType);
};
