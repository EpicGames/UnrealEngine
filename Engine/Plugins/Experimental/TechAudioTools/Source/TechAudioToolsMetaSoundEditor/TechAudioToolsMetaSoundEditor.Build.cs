// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class TechAudioToolsMetaSoundEditor : ModuleRules
{
    public TechAudioToolsMetaSoundEditor(ReadOnlyTargetRules Target) : base(Target)
    {
        PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

        PublicDependencyModuleNames.AddRange(
            new string[]
            {
	            "Blutility",
                "Core",
				"MetasoundEditor",
				"MetasoundEngine",
				"MetasoundFrontend",
				"ModelViewViewModel",
				"ModelViewViewModelEditor",
				"TechAudioTools",
				"TechAudioToolsMetaSound",
				"UnrealEd",
            }
        );

        PrivateDependencyModuleNames.AddRange(
            new string[]
            {
                "CoreUObject",
                "Engine",
            }
        );
    }
}
