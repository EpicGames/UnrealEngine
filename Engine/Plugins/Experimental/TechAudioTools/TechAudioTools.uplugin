{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "TechAudioTools",
	"Description": "A collection of audio-related tools and utilities.",
	"Category": "Audio",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"CanContainContent": true,
	"IsBetaVersion": true,
	"IsExperimentalVersion": true,
	"Installed": false,
	"Modules": [
		{
			"Name": "TechAudioTools",
			"Type": "Runtime",
			"LoadingPhase": "Default"
		},
		{
			"Name": "TechAudioToolsMetaSound",
			"Type": "Runtime",
			"LoadingPhase": "Default"
		},
		{
			"Name": "TechAudioToolsMetaSoundEditor",
			"Type": "Editor",
			"LoadingPhase": "Default"
		}
	],
	"Plugins": [
		{
			"Name": "Metasound",
			"Enabled": true
		},
		{
			"Name": "ModelViewViewModel",
			"Enabled": true
		}
	]
}